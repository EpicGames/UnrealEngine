[SectionsToSave]
bCanSaveAllSections=True

[/Script/PerformanceCaptureWorkflow.PerformanceCaptureSettings]
StageRoot=/PerformanceCaptureWorkflow/Stage/BP_StageRoot.BP_StageRoot_C
ViewModelClass=/PerformanceCaptureWorkflow/Core/VM_PCapWorkflow.VM_PCapWorkflow_C
MocapManagerDefaultWorkflow=/PerformanceCaptureWorkflow/Core/DA_MocapManagerDefaultWorkflow.DA_MocapManagerDefaultWorkflow
WorkflowPhaseUIBaseClass=/PerformanceCaptureWorkflow/Core/EW_PCapPanelBase.EW_PCapPanelBase_C
MocapManagerUI=/PerformanceCaptureWorkflow/Core/EW_MocapManager.EW_MocapManager
DatabaseHelperClass=/PerformanceCaptureWorkflow/Database/EU_InternalDatabaseHelper.EU_InternalDatabaseHelper_C
AllowedCaptureCharacterActorClasses=/Script/PerformanceCaptureCore.CaptureCharacter
DisallowedCaptureCharacterActorClasses=/Script/PerformanceCaptureCore.CapturePerformer

[CoreRedirects]
+ClassRedirects=(OldName="/Script/PerformanceCaptureWorkflow.PCap_DataAsset",NewName="/Script/PerformanceCaptureWorkflow.PCapDataAsset")
+ClassRedirects=(OldName="/Script/PerformanceCaptureWorkflow.PCap_CharacterDataAsset",NewName="/Script/PerformanceCaptureWorkflow.PCapCharacterDataAsset")
+ClassRedirects=(OldName="/Script/PerformanceCaptureWorkflow.PCap_PropDataAsset",NewName="/Script/PerformanceCaptureWorkflow.PCapPropDataAsset")
+ClassRedirects=(OldName="/Script/PerformanceCaptureWorkflow.PCap_PerformerDataAsset",NewName="/Script/PerformanceCaptureWorkflow.PCapPerformerDataAsset")

+ClassRedirects=(OldName="/Script/PerformanceCaptureWorkflow.PCapMarkerVisualizer",NewName="/Script/LiveLink.LiveLinkMarkerVisualizer")
+ClassRedirects=(OldName="/Script/PerformanceCaptureWorkflow.LiveLinkDataPreview",NewName="/Script/LiveLink.LiveLinkDataPreview")
+ClassRedirects=(OldName="/Script/PerformanceCaptureWorkflow.LiveLinkDataPreviewComponent",NewName="/Script/LiveLink.LiveLinkDataPreviewComponent")

+PackageRedirects=(OldName="/PerformanceCaptureWorkflow/Visualizers/MI_JointAxisVisualizer_Blue", NewName="/LiveLink/Visualizers/MI_JointAxisVisualizer_Blue")
+PackageRedirects=(OldName="/PerformanceCaptureWorkflow/Visualizers/MI_JointAxisVisualizer_Green", NewName="/LiveLink/Visualizers/MI_JointAxisVisualizer_Green")
+PackageRedirects=(OldName="/PerformanceCaptureWorkflow/Visualizers/MI_JointAxisVisualizer_Red", NewName="/LiveLink/Visualizers/MI_JointAxisVisualizer_Red")
+PackageRedirects=(OldName="/PerformanceCaptureWorkflow/Visualizers/MI_JointAxisVisualizer_Yellow", NewName="/LiveLink/Visualizers/MI_JointAxisVisualizer_Yellow")
+PackageRedirects=(OldName="/PerformanceCaptureWorkflow/Visualizers/MI_MarkerAxisVisualizer_Blue", NewName="/LiveLink/Visualizers/MI_MarkerAxisVisualizer_Blue")
+PackageRedirects=(OldName="/PerformanceCaptureWorkflow/Visualizers/MI_MarkerAxisVisualizer_Green", NewName="/LiveLink/Visualizers/MI_MarkerAxisVisualizer_Green")
+PackageRedirects=(OldName="/PerformanceCaptureWorkflow/Visualizers/MI_MarkerAxisVisualizer_Red", NewName="/LiveLink/Visualizers/MI_MarkerAxisVisualizer_Red")
+PackageRedirects=(OldName="/PerformanceCaptureWorkflow/Visualizers/MI_MarkerAxisVisualizer_Yellow", NewName="/LiveLink/Visualizers/MI_MarkerAxisVisualizer_Yellow")
+PackageRedirects=(OldName="/PerformanceCaptureWorkflow/Visualizers/M_BoneVisualizer", NewName="/LiveLink/Visualizers/M_BoneVisualizer")
+PackageRedirects=(OldName="/PerformanceCaptureWorkflow/Visualizers/M_BoneVisualizerOverlay", NewName="/LiveLink/Visualizers/M_BoneVisualizerOverlay")
+PackageRedirects=(OldName="/PerformanceCaptureWorkflow/Visualizers/M_JointAxisVisualizer", NewName="/LiveLink/Visualizers/M_JointAxisVisualizer")
+PackageRedirects=(OldName="/PerformanceCaptureWorkflow/Visualizers/M_MarkerAxisVisualizer", NewName="/LiveLink/Visualizers/M_MarkerAxisVisualizer")
+PackageRedirects=(OldName="/PerformanceCaptureWorkflow/Visualizers/M_MarkerAxisVisualizer_NoDepth", NewName="/LiveLink/Visualizers/M_MarkerAxisVisualizer_NoDepth")
+PackageRedirects=(OldName="/PerformanceCaptureWorkflow/Visualizers/SM_Bone", NewName="/LiveLink/Visualizers/SM_Bone")
+PackageRedirects=(OldName="/PerformanceCaptureWorkflow/Visualizers/SM_Joint", NewName="/LiveLink/Visualizers/SM_Joint")
+PackageRedirects=(OldName="/PerformanceCaptureWorkflow/Visualizers/SM_JointAxis", NewName="/LiveLink/Visualizers/SM_JointAxis")
+PackageRedirects=(OldName="/PerformanceCaptureWorkflow/Visualizers/SM_LiveLinkCamera", NewName="/LiveLink/Visualizers/SM_LiveLinkCamera")
+PackageRedirects=(OldName="/PerformanceCaptureWorkflow/Visualizers/SM_MarkerAxis", NewName="/LiveLink/Visualizers/SM_MarkerAxis")
+PackageRedirects=(OldName="/PerformanceCaptureWorkflow/Visualizers/SM_MarkerBox", NewName="/LiveLink/Visualizers/SM_MarkerBox")
+PackageRedirects=(OldName="/PerformanceCaptureWorkflow/Visualizers/SM_MarkerSphere", NewName="/LiveLink/Visualizers/SM_MarkerSphere")
+PackageRedirects=(OldName="/PerformanceCaptureWorkflow/Visualizers/SM_TransformAxis", NewName="/LiveLink/Visualizers/SM_TransformAxis")

+ClassRedirects=(OldName="/Script/PerformanceCaptureWorkflow.PerformanceCaptureStageRoot",NewName="/Script/PerformanceCaptureWorkflowRuntime.PerformanceCaptureStageRoot")
+ClassRedirects=(OldName="/Script/PerformanceCaptureWorkflow.PCapPropComponent",NewName="/Script/PerformanceCaptureWorkflowRuntime.PCapPropComponent")
+ClassRedirects=(OldName="/Script/PerformanceCaptureWorkflow.PCapPropLiveLinkAnimInstance",NewName="/Script/PerformanceCaptureWorkflowRuntime.PCapPropLiveLinkAnimInstance")
+ClassRedirects=(OldName="/Script/PerformanceCaptureWorkflow.AnimNode_LiveLinkProp",NewName="/Script/PerformanceCaptureWorkflowRuntime.AnimNode_LiveLinkProp")
+PropertyRedirects=(OldName="/Script/PerformanceCaptureWorkflow.PCapBoneVisualiser.SkelmeshComponent",NewName="/Script/PerformanceCaptureWorkflow.PCapBoneVisualiser.SkinnedMeshComponent")