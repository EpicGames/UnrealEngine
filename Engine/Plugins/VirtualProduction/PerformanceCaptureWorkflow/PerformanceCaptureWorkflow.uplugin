{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.2",
	"FriendlyName": "Performance Capture Workflow",
	"Description": "Performance Capture In-Editor Workflow tools. Provides access to the Mocap Manager panel.",
	"Category": "Animation",
	"CreatedBy": "Epic Games",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "PerformanceCaptureWorkflow",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "PerformanceCaptureWorkflowRuntime",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "MultiUserClient",
			"Enabled": true
		},
		{
			"Name": "LiveLink",
			"Enabled": true
		},
		{
			"Name": "PerformanceCaptureCore",
			"Enabled": true
		},
		{
			"Name": "GeometryScripting",
			"Enabled": true
		},
		{
			"Name": "CommonUI",
			"Enabled": true
		},
		{
			"Name": "SkeletalMeshModelingTools",
			"Enabled": true
		},
		{
			"Name": "EditorScriptingUtilities",
			"Enabled": true
		},
		{
			"Name": "ModelViewViewModel",
			"Enabled": true
		},
		{
			"Name": "EngineAssetDefinitions",
			"Enabled": true
		},
		{
			"Name": "Takes",
			"Enabled": true
		},
		{
			"Name": "DirectoryPlaceholder",
			"Enabled": true
		},
		{
			"Name": "NamingTokens",
			"Enabled": true
		},
		{
			"Name": "IKRig",
			"Enabled": true
		},
		{
			"Name": "LiveLinkUnrealDevice",
			"Enabled": true
		},
		{
			"Name": "CaptureManagerEditor",
			"Enabled": true
		}
	]
}