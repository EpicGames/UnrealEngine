// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PerformanceCaptureWorkflow : ModuleRules
{
	public PerformanceCaptureWorkflow(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		ShortName = "PCWF";

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CommonUI",
				"PlacementMode"
			}
			);
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"DeveloperSettings",
				"Settings",
				"PerformanceCaptureCore",
				"LiveLink",
				"LiveLinkInterface",
				"IKRig", 
				"LiveLinkAnimationCore",
				"LevelSequence",
				"Projects",
				"InputCore",
				"EditorFramework",
				"MainFrame",
				"UnrealEd",
				"ToolMenus",
				"ModelViewViewModel",
				"UMG",
				"UMGEditor",
				"Blutility",
				"EditorSubsystem",
				"AnimationCore", 
				"DataTableEditor",
				"AssetDefinition",
				"EngineAssetDefinitions",
				"CinematicCamera",
				"TakeRecorderSources", 
				"TakesCore",
				"SubobjectDataInterface",
				"NamingTokens",
				"PerformanceCaptureWorkflowRuntime", 
				"WorkspaceMenuStructure",
				"MovieScene"
			}
			);
	}
}
