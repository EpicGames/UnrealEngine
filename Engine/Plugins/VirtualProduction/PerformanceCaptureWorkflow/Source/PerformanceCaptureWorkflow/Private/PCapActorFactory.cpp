// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCapActorFactory.h"

#include "PCapDatabase.h"
#include "Retargeter/IKRetargeter.h"
#include "Engine/SkeletalMesh.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/StaticMesh.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "AssetSelection.h"
#include "PCapPropComponent.h"
#include "PerformanceCapture.h"
#include "SubobjectDataSubsystem.h"
#include "Elements/Actor/ActorElementData.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Logging/MessageLog.h"
#include "RetargetComponent.h"
#include "Subsystems/PlacementSubsystem.h"

#define LOCTEXT_NAMESPACE "PCapActorFactory"

namespace UE::PCapActorFactory
{

	/** Find and return an existing Component of type TComponent or add and return a new one if not. */
	UActorComponent* AddComponent(TSubclassOf<UActorComponent> ComponentClass, AActor* OnActor, bool bLogMessageOnError = true)
	{
		if(ComponentClass == nullptr)
		{
			return nullptr;
		}

		USubobjectDataSubsystem* SubobjectSubsystem = GEngine->GetEngineSubsystem<USubobjectDataSubsystem>();
		TArray<FSubobjectDataHandle> SubObjectHandles;
		SubobjectSubsystem->GatherSubobjectData(OnActor, SubObjectHandles);
		FSubobjectDataHandle RootHandle = SubObjectHandles[0]; //The actor's root component is always the first handle in the array
		FAddNewSubobjectParams NewObjectParams;
		NewObjectParams.ParentHandle = RootHandle;
		NewObjectParams.NewClass = ComponentClass;
		NewObjectParams.bConformTransformToParent = true;

		FText OutFailText;
		FSubobjectDataHandle NewComponentHandle = SubobjectSubsystem->AddNewSubobject(NewObjectParams, OutFailText);

		UActorComponent* Component = nullptr;
		if(NewComponentHandle.IsValid())
		{
			// GetObject returns a const* so need to cast that away to make this usable.
			Component = const_cast<UActorComponent*>(NewComponentHandle.GetData()->GetObject<UActorComponent>());
		}
		else if(bLogMessageOnError)
		{
			FFormatNamedArguments FormatArguments;
			FormatArguments.Add(TEXT("Actor"), FText::FromString(OnActor->GetName()));
			FormatArguments.Add(TEXT("ComponentType"), FText::FromString(ComponentClass->GetName()));
			FormatArguments.Add(TEXT("ErrorMessage"), OutFailText);

			FText Message = FText::Format(
				LOCTEXT("PCapActorFactoryAddComponentFail", "Unable to add Component {ComponentType} to {Actor}, error: {ErrorMessage}"),
				FormatArguments);

			FMessageLog(FPerformanceCaptureModule::MessageLogName).Error(Message);
		}

		return Component;
	}

} // namespace UE::PCapActorFactory

/*------------------------------------------------------------------------------
	Custom Actor Factory for Character Data Asset implementation.
------------------------------------------------------------------------------*/
UClass* UPCapCharacterActorFactory::GetDefaultActorClass(const FAssetData& AssetData)
{
	if(AssetData.IsValid())
	{
		if(UPCapCharacterDataAsset* CharacterDataAsset = Cast<UPCapCharacterDataAsset>(AssetData.GetAsset()))
		{
			if(UClass* Class = CharacterDataAsset->CaptureCharacterClass.LoadSynchronous())
			{
				return Class;
			}	
		}
	}
	return nullptr;
}

bool UPCapCharacterActorFactory::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	if(AssetData.IsValid())
	{
		if(UPCapCharacterDataAsset* CharacterDataAsset = Cast<UPCapCharacterDataAsset>(AssetData.GetAsset()))
		{
			const UClass* Class = CharacterDataAsset->CaptureCharacterClass.LoadSynchronous();
			const USkeletalMesh* SkeletalMesh = CharacterDataAsset->SkeletalMesh.LoadSynchronous();

			if(Class && SkeletalMesh)
			{
				return true;
			}	
		}
	}
	OutErrorMsg = NSLOCTEXT("Performance Capture", "CanCreateActorFrom_PerformerAsset", "Asset does not contain valid Capture Character Class or Skeletal Mesh");
	return false;
}

void UPCapCharacterActorFactory::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

	const UPCapCharacterDataAsset* CharacterAsset = Cast<UPCapCharacterDataAsset>(Asset);
	ASkeletalMeshActor* AsSkeletalMeshActor = Cast<ASkeletalMeshActor>(NewActor);

	if(AsSkeletalMeshActor != nullptr && CharacterAsset != nullptr && !AsSkeletalMeshActor->IsA<ACapturePerformer>())
	{
		TObjectPtr<URetargetComponent> RetargetComponent = AsSkeletalMeshActor->GetComponentByClass<URetargetComponent>();

		if(RetargetComponent == nullptr)
		{
			RetargetComponent = Cast<URetargetComponent>(UE::PCapActorFactory::AddComponent(URetargetComponent::StaticClass(), AsSkeletalMeshActor));
		}

		if(RetargetComponent != nullptr)
		{
			USkeletalMesh* CharacterMesh = CharacterAsset->SkeletalMesh.LoadSynchronous();
			AsSkeletalMeshActor->GetSkeletalMeshComponent()->SetSkeletalMeshAsset(CharacterMesh);
			RetargetComponent->SetRetargetAsset(Cast<UIKRetargeter>(CharacterAsset->Retargeter.LoadSynchronous()));

			if(SpawnedPerformerHandle.IsSet()) //Entry here is only once per placing action.
			{
				ACapturePerformer* SourcePerformer = Cast<ACapturePerformer>(ActorElementDataUtil::GetActorFromHandle(SpawnedPerformerHandle));
				RetargetComponent->SourcePerformer = SourcePerformer;
				AsSkeletalMeshActor->GetSkeletalMeshComponent()->SetReceivesDecals(false);
			}
		}
	}
	SpawnedPerformerHandle.Release();
}

TArray<FTypedElementHandle> UPCapCharacterActorFactory::PlaceAsset(const FAssetPlacementInfo& InPlacementInfo, const FPlacementOptions& InPlacementOptions)
{
	TArray<FTypedElementHandle> PlacedActorHandles;
	FActorSpawnParameters SpawnParams;
	SpawnParams.Name = InPlacementInfo.NameOverride;
	SpawnParams.ObjectFlags = InPlacementOptions.bIsCreatingPreviewElements ? EObjectFlags::RF_Transient : EObjectFlags::RF_Transactional;
	SpawnParams.bTemporaryEditorActor = InPlacementOptions.bIsCreatingPreviewElements;

	AActor* NewActor = CreateActor(InPlacementInfo.AssetToPlace.GetAsset(), InPlacementInfo.PreferredLevel.Get(), InPlacementInfo.FinalizedTransform, SpawnParams);
	if (NewActor)
	{
		PlacedActorHandles.Add(UEngineElementsLibrary::AcquireEditorActorElementHandle(NewActor));

		// Run post placement steps
		if (InPlacementOptions.bIsCreatingPreviewElements)
		{
			NewActor->SetActorEnableCollision(false);
		}
	}

	//Get Performer Asset and Attempt to spawn that too. 
	const UPCapCharacterDataAsset* CharacterData = Cast<UPCapCharacterDataAsset>(InPlacementInfo.AssetToPlace.GetAsset());
	
	if(	UPCapPerformerDataAsset* PerformerDataAsset = CharacterData->SourcePerformerAsset.LoadSynchronous()) //If the Character Asset has a valid performer, place/spawn 
	{
		UE::AssetPlacementUtil::FExtraPlaceAssetOptions PlacementOptions;
			TArray<FTypedElementHandle> NewPerformerHandles = PlaceAssetInCurrentLevel(PerformerDataAsset, PlacementOptions);
		PlacedActorHandles.Append(NewPerformerHandles);
		if(!NewPerformerHandles.IsEmpty())
		{
			//If successfully spawned, cache the created handle so we have a reference to the created Performer to use in Post operations.
			SpawnedPerformerHandle = NewPerformerHandles[0];
		}
	}
	return PlacedActorHandles;
}

/*------------------------------------------------------------------------------
	Custom Actor Factory for Performer Data Asset implementation.
------------------------------------------------------------------------------*/
UClass* UPCapPerformerActorFactory::GetDefaultActorClass(const FAssetData& AssetData)
{
	if(AssetData.IsValid())
	{
		if (UPCapPerformerDataAsset* PerformerDataAsset = Cast<UPCapPerformerDataAsset>(AssetData.GetAsset()))
		{
			if(UClass* Class = PerformerDataAsset->PerformerActorClass.LoadSynchronous())
			{
				return Class;
			}
		}
	}
	return nullptr;
}

bool UPCapPerformerActorFactory::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	if(AssetData.IsValid())
	{
		if(UPCapPerformerDataAsset* PerformerDataAsset = Cast<UPCapPerformerDataAsset>(AssetData.GetAsset()))
		{
			const UClass* Class = PerformerDataAsset->PerformerActorClass.LoadSynchronous();
			const USkeletalMesh* BaseSkeletalMesh = PerformerDataAsset->BaseSkeletalMesh.LoadSynchronous();
			const USkeletalMesh* ProportionedMesh = PerformerDataAsset->PerformerProportionedMesh.LoadSynchronous();
			if(Class && (BaseSkeletalMesh || ProportionedMesh))
			{
					return true;
			}	
		}
	}
	OutErrorMsg = NSLOCTEXT("Performance Capture", "CanCreateActorFrom_Asset", "Asset is missing a a valid Capture Performer Class or Skeletal Meshes");
	return false;
}

void UPCapPerformerActorFactory::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

	if(ACapturePerformer* NewPerformer = Cast<ACapturePerformer>(NewActor))
	{
		UPCapPerformerDataAsset* PerformerDataAsset = Cast<UPCapPerformerDataAsset>(Asset);
		if(PerformerDataAsset)
		{
			if(USkeletalMesh* PerformerScaledMesh = PerformerDataAsset->PerformerProportionedMesh.LoadSynchronous())
			{
				NewPerformer->SetMocapMesh(PerformerScaledMesh);
			}
			if(!PerformerDataAsset->PerformerProportionedMesh.IsValid() && PerformerDataAsset->BaseSkeletalMesh.IsValid()) //If there is not ProportionedMesh but a valid Base Mesh, use that instead
			{
				USkeletalMesh* PerformerBaseMesh = PerformerDataAsset->BaseSkeletalMesh.LoadSynchronous();
				NewPerformer->SetMocapMesh(PerformerBaseMesh);
			}

			NewPerformer->SetLiveLinkSubject(PerformerDataAsset->LiveLinkSubject);
			NewActor->SetActorLabel(PerformerDataAsset->PerformerName.ToString(), false);
			
			if(USkeletalMeshComponent* PerformerSkelMeshComponent = NewPerformer->GetSkeletalMeshComponent())
			{
				PerformerSkelMeshComponent->SetReceivesDecals(false);
				
				if(UMaterialInterface* SkelMeshMaterial = NewPerformer->GetSkeletalMeshComponent()->GetMaterial(0))
				{
					UMaterialInstanceDynamic* DynamicMaterial = PerformerSkelMeshComponent->CreateDynamicMaterialInstance(
						0, SkelMeshMaterial);
					PerformerSkelMeshComponent->SetMaterial(0, DynamicMaterial);
					DynamicMaterial->ClearParameterValues();
					DynamicMaterial->SetVectorParameterValue(FName(TEXT("PerformerColor")), PerformerDataAsset->PerformerColor);
				}
			}
		}
	}
}

/*------------------------------------------------------------------------------
	Custom Actor Factory for Prop Data Asset implementation.
------------------------------------------------------------------------------*/
UClass* UPCapPropActorFactory::GetDefaultActorClass(const FAssetData& AssetData)
{
	if(AssetData.IsValid())
	{
		if(UPCapPropDataAsset* PropDataAsset = Cast<UPCapPropDataAsset>(AssetData.GetAsset()))
		{
			if(UClass* CustomClass =  PropDataAsset->CustomPropClass.Get())
			{
				return CustomClass;
			}
			
			if(PropDataAsset->PropSkeletalMesh.IsValid())
			{
				if(UClass* SkeletalMeshClass = ASkeletalMeshActor::StaticClass())
				{
					return SkeletalMeshClass;
				}
			}

			if(PropDataAsset->PropStaticMesh.IsValid())
			{
				if(UClass* StaticMeshClass = AStaticMeshActor::StaticClass())
				{
					return StaticMeshClass;
				}
			}
		}
	}
	return nullptr;
}

bool UPCapPropActorFactory::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	if(AssetData.IsValid())
	{
		if(UPCapPropDataAsset* PropDataAsset = Cast<UPCapPropDataAsset>(AssetData.GetAsset()))
		{
			const UStaticMesh* StaticMesh = PropDataAsset->PropStaticMesh.LoadSynchronous();
			const USkeletalMesh* SkeletalMesh = PropDataAsset->PropSkeletalMesh.LoadSynchronous();;
			const UClass* CustomProp = PropDataAsset->CustomPropClass.LoadSynchronous();
			const UClass* PropComponent = PropDataAsset->PropComponentClass.LoadSynchronous();
						
			if((SkeletalMesh || StaticMesh || CustomProp) && PropComponent )
			{
				return true;
			}
		}
	}
	OutErrorMsg = NSLOCTEXT("Performance Capture", "CanCreateActorFrom_PropAsset", "Asset does not contain valid Static Mesh or Skeletal Mesh");
	return false;
}

void UPCapPropActorFactory::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);
	
	if(NewActor && Asset)
	{
		if(UPCapPropDataAsset* PropDataAsset = Cast<UPCapPropDataAsset>(Asset))
		{
			if(AStaticMeshActor* NewStaticMeshProp = Cast<AStaticMeshActor>(NewActor))
			{		
				if(UStaticMesh* StaticMesh = PropDataAsset->PropStaticMesh.LoadSynchronous())
				{
					NewStaticMeshProp->GetStaticMeshComponent()->SetStaticMesh(PropDataAsset->PropStaticMesh.LoadSynchronous());
					NewStaticMeshProp->GetStaticMeshComponent()->SetReceivesDecals(false);
					if(PropDataAsset->PropName.IsValid()) //If the user has filled out a valid FName use that for the actor label
					{
						NewStaticMeshProp->SetActorLabel(PropDataAsset->PropName.ToString(), false);
					}
					else //Otherwise use the name of the DataAsset
					{
						NewStaticMeshProp->SetActorLabel(PropDataAsset->GetName(), false);
					}
				}
				NewStaticMeshProp->SetActorHiddenInGame(PropDataAsset->bHiddenInGame);
			}
			
			if(ASkeletalMeshActor* NewSkeletalMeshProp = Cast<ASkeletalMeshActor>(NewActor))
			{
				if(USkeletalMesh* SkeletalMesh = PropDataAsset->PropSkeletalMesh.LoadSynchronous())
				{
					NewSkeletalMeshProp->GetSkeletalMeshComponent()->SetSkeletalMeshAsset(PropDataAsset->PropSkeletalMesh.LoadSynchronous());
					NewSkeletalMeshProp->GetSkeletalMeshComponent()->SetReceivesDecals(false);
					if(PropDataAsset->PropName.IsValid()) //If the user has filled out a valid FName use that for the actor label
					{
						NewSkeletalMeshProp->SetActorLabel(PropDataAsset->PropName.ToString(), false);
					}
					else //Otherwise use the name of the DataAsset
					{
						NewSkeletalMeshProp->SetActorLabel(PropDataAsset->GetName(), false);
					}
					
				}
				NewSkeletalMeshProp->SetActorHiddenInGame(PropDataAsset->bHiddenInGame);
			}
			//Only add Subcomponents for the non-transient actor because transient means it is still being placed. Also, don't add the prop component for custom classes
			if(NewActor->HasAnyFlags(RF_Transient) != true && NewActor->GetClass()!= (PropDataAsset->CustomPropClass.Get())) 
			{
				USubobjectDataSubsystem* SubSystem = GEngine->GetEngineSubsystem<USubobjectDataSubsystem>();
				TObjectPtr<UPCapPropComponent> NewPropComponent =  Cast<UPCapPropComponent>(NewActor->GetComponentByClass(UPCapPropComponent::StaticClass()));
				if(!NewPropComponent)
				{
					TArray<FSubobjectDataHandle> Handles;
					SubSystem->GatherSubobjectData(NewActor, Handles);

					FSubobjectDataHandle Handle = Handles[0]; //Root Component is always first entry in handles array

					FAddNewSubobjectParams NewObjectParams;

					NewObjectParams.ParentHandle = Handle;
					NewObjectParams.NewClass = PropDataAsset->PropComponentClass.Get();
					NewObjectParams.bConformTransformToParent = true;

					FText FailureReason;
					FSubobjectDataHandle NewPropComponentHandle = SubSystem->AddNewSubobject(NewObjectParams, FailureReason);
				}
				
				UE_LOGF(LogPCap, Display, "New PCap Prop Component added to %ls", *NewActor->GetActorLabel())
			}

			//If the new actor has been spawned from a BP class, check it has a Prop component and set the Live Link Subject
			//Or, if a new prop component has just been added above set the subject, offset and spawning data asset for later retrieval. 
			
			NewActor->SetActorLabel(PropDataAsset->PropName.ToString(), false);
			
			if(UPCapPropComponent* PropComponent = Cast<UPCapPropComponent>(NewActor->GetComponentByClass(UPCapPropComponent::StaticClass())))
			{
				PropComponent->SetLiveLinkSubject(PropDataAsset->LiveLinkSubject);
				PropComponent->SetOffsetTransform(PropDataAsset->PropOffsetTransform);
				PropComponent->SpawningDataAsset = FAssetData(PropDataAsset).PackageName; //Add the package name of the spawning data asset to the prop component so the toolset can locate the data asset for a given prop
			}
			else
			{
				UE_LOGF(LogPCap, Display, "Prop spawned from data asset with no valid PcapProp component. %ls", *NewActor->GetActorLabel())
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE