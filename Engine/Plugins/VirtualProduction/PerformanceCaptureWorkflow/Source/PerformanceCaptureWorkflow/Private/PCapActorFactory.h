// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ActorFactories/ActorFactory.h"
#include "Elements/Framework/TypedElementHandle.h"
#include "PCapActorFactory.generated.h"

class ACapturePerformer;
struct FTypedElementHandle;

/**
 * Actor factory for spawning Capture Character from the PCapCharacterDataAsset.
 */

UCLASS()
class UPCapCharacterActorFactory : public UActorFactory
{
	GENERATED_BODY()

	FTypedElementHandle SpawnedPerformerHandle;
	
	/** Initialize NewActorClass if necessary, and return that class. */
	virtual UClass* GetDefaultActorClass( const FAssetData& AssetData ) override;

	virtual bool CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg ) override;

	virtual void PostSpawnActor( UObject* Asset, AActor* NewActor ) override;

	virtual TArray<FTypedElementHandle> PlaceAsset(const FAssetPlacementInfo& InPlacementInfo, const FPlacementOptions& InPlacementOptions) override;
};

UCLASS()
class UPCapPerformerActorFactory : public UActorFactory
{
	GENERATED_BODY()

	/** Initialize NewActorClass if necessary, and return that class. */
	virtual UClass* GetDefaultActorClass( const FAssetData& AssetData ) override;

	virtual bool CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg ) override;

	virtual void PostSpawnActor( UObject* Asset, AActor* NewActor ) override;
};

UCLASS()

class UPCapPropActorFactory : public UActorFactory
{
	GENERATED_BODY()

	/** Initialize NewActorClass if necessary, and return that class. */
	virtual UClass* GetDefaultActorClass(const FAssetData& AssetData) override;

	virtual bool CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg) override;

	virtual void PostSpawnActor( UObject* Asset, AActor* NewActor ) override;
};