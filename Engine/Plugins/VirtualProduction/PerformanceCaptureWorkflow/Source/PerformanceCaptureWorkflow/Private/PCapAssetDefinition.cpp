// Copyright Epic Games, Inc. All Rights Reserved.


#include "PCapAssetDefinition.h"
#include "PerformanceCaptureStyle.h"
#include "Misc/FileHelper.h"
#include "Misc/MessageDialog.h"
#include "Settings/EditorLoadingSavingSettings.h"

#define LOCTEXT_NAMESPACE "UAssetDefinition_PerformanceCapture"

/*------------------------------------------------------------------------------
	UPCapDataAsset Performer implementation.
------------------------------------------------------------------------------*/

const FSlateBrush* UAssetDefinition_PerformerDataAsset::GetIconBrush(const FAssetData& InAssetData, const FName InClassName) const
{
	return FPerformanceCaptureStyle::Get().GetBrush("ClassIcon.PCapPerformerDataAsset");
}

const FSlateBrush* UAssetDefinition_PerformerDataAsset::GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const
{
	return FPerformanceCaptureStyle::Get().GetBrush("ClassThumbnail.PCapPerformerDataAsset");
}

/*------------------------------------------------------------------------------
	UPCapDataAsset Character implementation.
------------------------------------------------------------------------------*/

const FSlateBrush* UAssetDefinition_CharacterDataAsset::GetIconBrush(const FAssetData& InAssetData, const FName InClassName) const
{
	return FPerformanceCaptureStyle::Get().GetBrush("ClassIcon.PCapCharacterDataAsset");
}

const FSlateBrush* UAssetDefinition_CharacterDataAsset::GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const
{
	return FPerformanceCaptureStyle::Get().GetBrush("ClassThumbnail.PCapCharacterDataAsset");
}

/*------------------------------------------------------------------------------
	UPCapDataAsset Prop implementation.
------------------------------------------------------------------------------*/
const FSlateBrush* UAssetDefinition_PropDataAsset::GetIconBrush(const FAssetData& InAssetData, const FName InClassName) const
{
	return FPerformanceCaptureStyle::Get().GetBrush("ClassIcon.PCapPropDataAsset");
}

const FSlateBrush* UAssetDefinition_PropDataAsset::GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const
{
	return FPerformanceCaptureStyle::Get().GetBrush("ClassThumbnail.PCapPropDataAsset");
}

/*------------------------------------------------------------------------------
	UPCapDataAsset Session Template implementation.
------------------------------------------------------------------------------*/
const FSlateBrush* UAssetDefinition_SessionTemplateAsset::GetIconBrush(const FAssetData& InAssetData, const FName InClassName) const
{
	return FPerformanceCaptureStyle::Get().GetBrush("ClassIcon.PCapSessionTemplate");
}

const FSlateBrush* UAssetDefinition_SessionTemplateAsset::GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const
{
	return FPerformanceCaptureStyle::Get().GetBrush("ClassThumbnail.PCapSessionTemplate");
}

/*------------------------------------------------------------------------------
	UPCapData Workflow Customization implementation.
------------------------------------------------------------------------------*/
const FSlateBrush* UAssetDefinition_PCapDataWorkflowCustomization::GetIconBrush(const FAssetData& InAssetData, const FName InClassName) const
{
	return FPerformanceCaptureStyle::Get().GetBrush("ClassIcon.PCapWorkflowCustomization");
}

const FSlateBrush* UAssetDefinition_PCapDataWorkflowCustomization::GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const
{
	return FPerformanceCaptureStyle::Get().GetBrush("ClassThumbnail.PCapWorkflowCustomization");
}

#undef LOCTEXT_NAMESPACE