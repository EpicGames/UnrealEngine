// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinition_DataAsset.h"
#include "PCapDataTable.h"
#include "PCapDatabase.h"
#include "PCapSessionTemplate.h"
#include "PCapWorkflowCustomization.h"
#include "Table/AssetDefinition_DataTable.h"

#include "PCapAssetDefinition.generated.h"

/**
 * 
 */
UCLASS()
class PERFORMANCECAPTUREWORKFLOW_API UAssetDefinition_PCapDataTable : public UAssetDefinition_DataTable
{
	GENERATED_BODY()

	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "PCapDataTable", "PCap Data Table"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(57, 181, 74)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UPCapDataTable::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const FAssetCategoryPath Categories[] =
		{
			FAssetCategoryPath(
				FAssetCategoryPath(NSLOCTEXT("AssetDefinition", "PCapDataTable_AssetCategory", "Virtual Production")),
				NSLOCTEXT("AssetDefinition", "PCapDataTable_CategorySection", "Performance Capture"), ECategoryMenuType::Section)
		};

		return Categories;
	}
	virtual bool CanImport() const override { return true; }
};

UCLASS()
class UAssetDefinition_PCapDataAsset : public UAssetDefinition_DataAsset
{
	GENERATED_BODY()
public:
	
	// UAssetDefinition Implementation
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "PCapDataAsset", "PCap DataAsset"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(161, 57, 191)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UPCapDataAsset::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const FAssetCategoryPath Categories[] =
		{
			FAssetCategoryPath(
				FAssetCategoryPath(NSLOCTEXT("AssetDefinition", "PCapDataAsset_AssetCategory", "Virtual Production")),
				NSLOCTEXT("AssetDefinition", "PCapDataAsset_CategorySection", "Performance Capture"), ECategoryMenuType::Section)
		};

		return Categories;
	}
	// UAssetDefinition End
};

UCLASS()
class UAssetDefinition_PerformerDataAsset : public UAssetDefinition_PCapDataAsset
{
	GENERATED_BODY()
public:
	
	// UAssetDefinition Implementation
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "PCapPerformerDataAsset", "PCap Performer Asset"); }
	virtual const FSlateBrush* GetIconBrush(const FAssetData& InAssetData, const FName InClassName) const override;
	virtual const FSlateBrush* GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UPCapPerformerDataAsset::StaticClass(); }
	// UAssetDefinition End
};

UCLASS()
class UAssetDefinition_CharacterDataAsset : public UAssetDefinition_PCapDataAsset
{
	GENERATED_BODY()
public:
	
	// UAssetDefinition Implementation
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "PCapCharacterDataAsset", "PCap Character Asset"); }
	virtual const FSlateBrush* GetIconBrush(const FAssetData& InAssetData, const FName InClassName) const override;
	virtual const FSlateBrush* GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UPCapCharacterDataAsset::StaticClass(); }
	// UAssetDefinition End
};

UCLASS()
class UAssetDefinition_PropDataAsset : public UAssetDefinition_PCapDataAsset
{
	GENERATED_BODY()
public:
	
	// UAssetDefinition Implementation
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "PCapPropDataAsset", "PCap Prop Asset"); }
	virtual const FSlateBrush* GetIconBrush(const FAssetData& InAssetData, const FName InClassName) const override;
	virtual const FSlateBrush* GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UPCapPropDataAsset::StaticClass(); }
	// UAssetDefinition End
};

UCLASS()
class UAssetDefinition_SessionTemplateAsset : public UAssetDefinition_PCapDataAsset
{
	GENERATED_BODY()
public:
	
	// UAssetDefinition Implementation
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "PCapSessionTemplate", "PCap Session Template"); }
	virtual const FSlateBrush* GetIconBrush(const FAssetData& InAssetData, const FName InClassName) const override;
	virtual const FSlateBrush* GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UPCapSessionTemplate::StaticClass(); }
	// UAssetDefinition End
};
UCLASS()
class UAssetDefinition_PCapDataWorkflowCustomization : public UAssetDefinition_DataAsset
{
	GENERATED_BODY()
public:
	
	// UAssetDefinition Implementation
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "PCapDataAsset", "PCap DataAsset"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(161, 57, 191)); }
	virtual const FSlateBrush* GetIconBrush(const FAssetData& InAssetData, const FName InClassName) const override;
	virtual const FSlateBrush* GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UPCapWorkflowCustomization::StaticClass(); }
	// UAssetDefinition End
};