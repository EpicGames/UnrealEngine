// Copyright Epic Games, Inc. All Rights Reserved.


#include "PCapAssetFactory.h"
#include "AnalyticsEventAttribute.h"
#include "EngineAnalytics.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "PCapDatabase.h"
#include "PCapWorkflowCustomization.h"
#include "AssetToolsModule.h"
#include "ClassViewerModule.h"
#include "ClassViewerFilter.h"
#include "IAssetTools.h"
#include "PCapDataTable.h"
#include "StructViewerFilter.h"
#include "StructViewerModule.h"
#include "Editor.h"
#include "PerformanceCaptureStyle.h"

#include "Modules/ModuleManager.h"

#include "Widgets/SWindow.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Styling/AppStyle.h"
#include "Input/Reply.h"
#include "Kismet2/SClassPickerDialog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCapAssetFactory)

/*------------------------------------------------------------------------------
	UPCap_DataAssetFactory implementation.
------------------------------------------------------------------------------*/

#define LOCTEXT_NAMESPACE "PerformanceCaptureDatatableFactory"

UPCapDataTableFactory::UPCapDataTableFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UPCapDataTable::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
	Struct = FPCapRecordBase::StaticStruct();
}

bool UPCapDataTableFactory::ConfigureProperties()
{
	class FDataTableStructFilter : public IStructViewerFilter
	{
	public:
		virtual bool IsStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const UScriptStruct* InStruct, TSharedRef<FStructViewerFilterFuncs> InFilterFuncs) override
		{
			return (InStruct->IsChildOf(FPCapRecordBase::StaticStruct()) && (InStruct !=FPCapRecordBase::StaticStruct()));
		}

		virtual bool IsUnloadedStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const FSoftObjectPath& InStructPath, TSharedRef<FStructViewerFilterFuncs> InFilterFuncs) override
		{
			// Unloaded structs are always User Defined Structs, and User Defined Structs are always allowed
			// They will be re-validated by IsStructAllowed once loaded during the pick
			return false;
		}
	};

	class FDataTableFactoryUI : public TSharedFromThis<FDataTableFactoryUI>
	{
	public:
		FReply OnCreate()
		{
			check(ResultStruct);
			if (PickerWindow.IsValid())
			{
				PickerWindow->RequestDestroyWindow();
			}
			return FReply::Handled();
		}

		FReply OnCancel()
		{
			ResultStruct = nullptr;
			if (PickerWindow.IsValid())
			{
				PickerWindow->RequestDestroyWindow();
			}
			return FReply::Handled();
		}

		bool IsStructSelected() const
		{
			return ResultStruct != nullptr;
		}

		void OnPickedStruct(const UScriptStruct* ChosenStruct)
		{
			ResultStruct = ChosenStruct;
			StructPickerAnchor->SetIsOpen(false);
		}

		FText OnGetComboTextValue() const
		{
			return ResultStruct
				? FText::AsCultureInvariant(ResultStruct->GetName())
				: LOCTEXT("None", "None");
		}

		TSharedRef<SWidget> GenerateStructPicker()
		{
			FStructViewerModule& StructViewerModule = FModuleManager::LoadModuleChecked<FStructViewerModule>("StructViewer");

			// Fill in options
			FStructViewerInitializationOptions Options;
			Options.Mode = EStructViewerMode::StructPicker;
			Options.StructFilter = MakeShared<FDataTableStructFilter>();

			return
				SNew(SBox)
				.WidthOverride(330.0f)
				[
					SNew(SVerticalBox)

					+SVerticalBox::Slot()
					.FillHeight(1.0f)
					.MaxHeight(500)
					[
						SNew(SBorder)
						.Padding(4)
						.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
						[
							StructViewerModule.CreateStructViewer(Options, FOnStructPicked::CreateSP(this, &FDataTableFactoryUI::OnPickedStruct))
						]
					]
				];
		}

		const UScriptStruct* OpenStructSelector()
		{
			FStructViewerModule& StructViewerModule = FModuleManager::LoadModuleChecked<FStructViewerModule>("StructViewer");
			ResultStruct = nullptr;

			// Fill in options
			FStructViewerInitializationOptions Options;
			Options.Mode = EStructViewerMode::StructPicker;
			Options.StructFilter = MakeShared<FDataTableStructFilter>();

			PickerWindow = SNew(SWindow)
				.Title(LOCTEXT("PCapDataTableFactoryOptions", "Pick Performance Capture Table Row Structure"))
				.ClientSize(FVector2D(350, 100))
				.SupportsMinimize(false)
				.SupportsMaximize(false)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("Menu.Background"))
					.Padding(10)
					[
						SNew(SVerticalBox)
						+SVerticalBox::Slot()
						.AutoHeight()
						[
							SAssignNew(StructPickerAnchor, SComboButton)
							.ContentPadding(FMargin(2,2,2,1))
							.MenuPlacement(MenuPlacement_BelowAnchor)
							.ButtonContent()
							[
								SNew(STextBlock)
								.Text(this, &FDataTableFactoryUI::OnGetComboTextValue)
							]
							.OnGetMenuContent(this, &FDataTableFactoryUI::GenerateStructPicker)
						]
						+SVerticalBox::Slot()
						.HAlign(HAlign_Right)
						.AutoHeight()
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SButton)
								.Text(LOCTEXT("OK", "OK"))
								.IsEnabled(this, &FDataTableFactoryUI::IsStructSelected)
								.OnClicked(this, &FDataTableFactoryUI::OnCreate)
							]
							+SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SButton)
								.Text(LOCTEXT("Cancel", "Cancel"))
								.OnClicked(this, &FDataTableFactoryUI::OnCancel)
							]
						]
					]
				];

			GEditor->EditorAddModalWindow(PickerWindow.ToSharedRef());
			PickerWindow.Reset();

			return ResultStruct;
		}

	private:
		TSharedPtr<SWindow> PickerWindow;
		TSharedPtr<SComboButton> StructPickerAnchor;
		const UScriptStruct* ResultStruct = nullptr;
	};

	TSharedRef<FDataTableFactoryUI> StructSelector = MakeShared<FDataTableFactoryUI>();
	Struct = StructSelector->OpenStructSelector();

	return Struct != nullptr;
}

FText UPCapDataTableFactory::GetDisplayName() const
{
	return LOCTEXT("PCapDataTable_DisplayName", "PCap Data Table");
}

FText UPCapDataTableFactory::GetToolTip() const
{
	return LOCTEXT("PerformanceCaptureDatatable_Tooltip", "Create a datatable for Performance Capture data management");
}

bool UPCapDataTableFactory::ShouldShowInNewMenu() const
{
	return true;
}

uint32 UPCapDataTableFactory::GetMenuCategories() const
{
	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	return AssetTools.FindAdvancedAssetCategory("VirtualProduction");
}

FName UPCapDataTableFactory::GetNewAssetThumbnailOverride() const
{
	return "ClassThumbnail.PCapDataTable";
}

UDataTable* UPCapDataTableFactory::MakeNewDataTable(UObject* InParent, FName Name, EObjectFlags Flags)
{
	UPCapDataTable* NewTable = NewObject<UPCapDataTable>(InParent, Name, Flags);

	if (NewTable && FEngineAnalytics::IsAvailable())
	{
		TArray<FAnalyticsEventAttribute> Attributes;
		Attributes.Reserve(3);
		Attributes.Emplace(TEXT("Action"), TEXT("AssetCreated"));
		Attributes.Emplace(TEXT("AssetClass"), UPCapDataTable::StaticClass()->GetName());
		Attributes.Emplace(TEXT("RowStruct"), Struct ? Struct->GetName() : TEXT("None"));
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.PerformanceCapture"), Attributes);
	}

	return NewTable;
}

#undef  LOCTEXT_NAMESPACE

/*------------------------------------------------------------------------------
	UPCapData Class Filter implementation.
------------------------------------------------------------------------------*/

class FPCapClassParentFilter : public IClassViewerFilter
{
public:
	FPCapClassParentFilter()
	: DisallowedClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists | CLASS_HideDropDown) {}

	/** All children of these classes will be included unless filtered out by another setting. */
	TSet<const UClass*> AllowedChildrenOfClasses;

	/** Disallowed class flags. */
	EClassFlags DisallowedClassFlags;

	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override
	{
		return !InClass->HasAnyClassFlags(DisallowedClassFlags)
			&& InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InClass) != EFilterReturn::Failed;
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef<const IUnloadedBlueprintData> InUnloadedClassData, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override
	{
		return !InUnloadedClassData->HasAnyClassFlags(DisallowedClassFlags)
			&& InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InUnloadedClassData) != EFilterReturn::Failed;
	}
};

/*------------------------------------------------------------------------------
	UPCapDataAssetFactory implementation.
------------------------------------------------------------------------------*/

#define LOCTEXT_NAMESPACE "PerformanceCaptureDataAssetFactory"

UPCap_DataAssetFactory::UPCap_DataAssetFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UPCapDataAsset::StaticClass();
}

bool UPCap_DataAssetFactory::ConfigureProperties()
{
	// nullptr the DataAssetClass so we can check for selection
	DataAssetClass = nullptr;

	// Load the classviewer module to display a class picker
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	// Fill in options
	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;

	TSharedPtr<FPCapClassParentFilter> Filter = MakeShareable(new FPCapClassParentFilter);
	Filter->AllowedChildrenOfClasses.Add(UPCapDataAsset::StaticClass());

	Options.ClassFilters.Add(Filter.ToSharedRef());

	const FText TitleText = LOCTEXT("CreatePCapDataAsset", "Pick Class For Performance Capture Data Asset");
	UClass* ChosenClass = nullptr;
	const bool bPressedOk = SClassPickerDialog::PickClass(TitleText, Options, ChosenClass, UPCapDataAsset::StaticClass());

	if (bPressedOk)
	{
		DataAssetClass = ChosenClass;
	}
	return bPressedOk;
}

UObject* UPCap_DataAssetFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UPCapDataAsset* NewDataAsset = nullptr;

	if(DataAssetClass != nullptr)
	{
		NewDataAsset = NewObject<UPCapDataAsset>(InParent, DataAssetClass, Name, Flags | RF_Transactional, Context);
	}
	check(NewDataAsset);

	if (NewDataAsset && FEngineAnalytics::IsAvailable())
	{
		const UClass* CreatedClass = DataAssetClass ? DataAssetClass.Get() : NewDataAsset->GetClass();
		TArray<FAnalyticsEventAttribute> Attributes;
		Attributes.Reserve(2);
		Attributes.Emplace(TEXT("Action"), TEXT("AssetCreated"));
		Attributes.Emplace(TEXT("AssetClass"), CreatedClass->GetName());
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.PerformanceCapture"), Attributes);
	}

	return NewDataAsset;
}

uint32 UPCap_DataAssetFactory::GetMenuCategories() const
{
	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	return AssetTools.FindAdvancedAssetCategory("VirtualProduction");
}

#undef  LOCTEXT_NAMESPACE