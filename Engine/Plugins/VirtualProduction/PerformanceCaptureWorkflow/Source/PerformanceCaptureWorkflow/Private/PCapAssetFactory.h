// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PCapDatabase.h"
#include "Factories/DataTableFactory.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Factories/Factory.h"
#include "PCapAssetFactory.generated.h"

/**
 * Custom PCapDataTable Factory 
 */
UCLASS(BlueprintType)
class UPCapDataTableFactory : public UDataTableFactory
{
	GENERATED_BODY()
	
public:

	UPCapDataTableFactory(const FObjectInitializer& ObjectInitializer);

	//~ Begin UFactory Interface
	virtual bool ConfigureProperties() override;
	virtual FText GetDisplayName() const override;
	virtual FText GetToolTip() const override;
	virtual bool ShouldShowInNewMenu() const override;
	virtual uint32 GetMenuCategories() const override;
	virtual FName GetNewAssetThumbnailOverride() const override;
	//~ Begin UFactory Interface

protected:
	
	virtual UDataTable* MakeNewDataTable(UObject* InParent, FName Name, EObjectFlags Flags) override;
};

UCLASS(hidecategories=Object, MinimalAPI)

class UPCap_DataAssetFactory : public UFactory
{
	GENERATED_BODY()
	
public:
	UPCap_DataAssetFactory(const FObjectInitializer& ObjectInitializer);
	
	UPROPERTY(EditAnywhere, Category=DataAsset)
	TSubclassOf<UPCapDataAsset> DataAssetClass;

	// UFactory interface
	virtual bool ConfigureProperties() override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual uint32 GetMenuCategories() const override;
	// End of UFactory interface

};