// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCapBPFunctionLibrary.h"
#include "TakesCore/Public/TakeRecorderSource.h"
#include "Engine/Engine.h"
#include "EngineUtils.h"


FString UPerformanceCaptureBPFunctionLibrary::SanitizeFileString(FString InString)
{
	FString BannedChars = "[]<>{}!\"$£%^&*()+=;:,.?\\/|'@#~ ";
	FString CleanString = InString;

	//BannedChars = BannedChars + FPaths::GetInvalidFileSystemChars(); //This probably causes duplicates but rather than consider each platform here, we'll rely on the core.
	TArray<FString> IgnoredChars;

	for(auto CharIt(BannedChars.CreateConstIterator()); CharIt; ++CharIt)
	{
		TCHAR Char = *CharIt;
		IgnoredChars.Add(FString(1, &Char));
	}
	// Remove the null terminator on the end
	IgnoredChars.RemoveAt(IgnoredChars.Num()-1, 1);
	
	for(FString IgnoredChar : IgnoredChars)
	{
		CleanString = CleanString.Replace(*IgnoredChar, TEXT(""), ESearchCase::IgnoreCase);
	}
	return CleanString;
}

FString UPerformanceCaptureBPFunctionLibrary::SanitizePathString(FString InString)
{
	{
		//This is the same as for file name strings but with allowed exception of "/" directory limiters
		//TODO: currently windows only!
		FString BannedChars = "[]<>{}!\"$£%^&*()+=;:,.?|'@#~ ";
		FString CleanString = InString;

		//BannedChars = BannedChars + FPaths::GetInvalidFileSystemChars(); //This probably causes duplicates but rather than consider each platform here, we'll rely on the core.
		TArray<FString> IgnoredChars;

		for(auto CharIt(BannedChars.CreateConstIterator()); CharIt; ++CharIt)
		{
			TCHAR Char = *CharIt;
			IgnoredChars.Add(FString(1, &Char));
		}
		// Remove the null terminator on the end
		IgnoredChars.RemoveAt(IgnoredChars.Num()-1, 1);
	
		for(FString IgnoredChar : IgnoredChars)
		{
			CleanString = CleanString.Replace(*IgnoredChar, TEXT(""), ESearchCase::IgnoreCase);
		}
		return CleanString;
	}
}

void UPerformanceCaptureBPFunctionLibrary::GetAllActorsWithComponent(const UObject* WorldContextObject, TSubclassOf<UActorComponent> Component, TArray<AActor*>& OutActors)
{
	QUICK_SCOPE_CYCLE_COUNTER(ITWBPFactory_GetAllActorsWithComponent);
	OutActors.Reset();
    
	if (!Component)
	{
		return;
	}

	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		for (FActorIterator It(World); It; ++It)
		{
			AActor* Actor = *It;
			if (Actor->GetComponentByClass(Component) != nullptr)
			{
				OutActors.Add(Actor);
			}
		}
	}
}
