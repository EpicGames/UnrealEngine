// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Editor/Blutility/Private/EditorFunctionLibrary.h"
#include "Components/ActorComponent.h"
#include "PCapBPFunctionLibrary.generated.h"

class UTakeRecorderSource;
/**
 * 
 */
UCLASS()
class PERFORMANCECAPTUREWORKFLOW_API UPerformanceCaptureBPFunctionLibrary : public UEditorFunctionLibrary
{
	GENERATED_BODY()
	
public:
	/**
	 * Returns a string stripped of the following characters []<>{}1!"$£%^&*()+=;:,.?/\|'@#~
	 * @param	InString String you wish to clean
	 * @return	Cleaned string
	 */
	UFUNCTION(BlueprintCallable, Category = "Performance Capture|Workflow",meta=(DisplayName="Sanitize File Name String"))
	static FString SanitizeFileString(FString InString);
	
	/**
	 * Returns a string stripped of the following characters []<>{}1!"$£%^&*()+=;:,.?\|'@#~ Note filepath delimiters are not excluded
	 * @param	InString String you wish to clean
	 * @return	Cleaned string 
	 */
	UFUNCTION(BlueprintCallable, Category = "Performance Capture|Workflow",meta=(DisplayName="Sanitize File Path String"))
	static FString SanitizePathString(FString InString);
	
	/** 
	* Find all Actors in the world containing at least one instance of the given component class. 
	* This is a very slow operation, as it will search over every actor in the world.
	* @param	Component	Class of component to find. Must be specified or result array will be empty.
	* @param	OutActors	Output array of Actors of the specified class.
	 */
	UFUNCTION(BlueprintCallable, Category="Performance Capture|Core",  meta=(WorldContext="WorldContextObject", DeterminesOutputType="Component", DynamicOutputParam="OutActors"))
	static void GetAllActorsWithComponent(const UObject* WorldContextObject, TSubclassOf<UActorComponent> Component, TArray<AActor*>& OutActors);
};
