// Copyright Epic Games, Inc. All Rights Reserved.


#include "PCapDataTable.h"
#include "PCapDatabase.h"
#include "DataTableEditorUtils.h"
#include "PerformanceCapture.h"
#include "Blueprint/BlueprintExceptionInfo.h"

UPCapDataTable::UPCapDataTable()
{
	RowStruct = FPCapRecordBase::StaticStruct();
	bStripFromClientBuilds = true; //Prevent any Performance Capture data from cooking into client builds as this is editor-only tooling
	
	OnDataTableChanged().AddUObject(this, &UPCapDataTable::DataTableModified);
}

UPCapDataTable::~UPCapDataTable()
{

}

void UPCapDataTable::DataTableModified() const
{
	OnDatatableModified.Broadcast();
}

bool UPCapDataTable::RemoveTableRow(FName RowName)
{
	return FDataTableEditorUtils::RemoveRow(this, RowName);
}

bool UPCapDataTable::DuplicateTableRow(FName SourceRow, FName NewRow)
{
	if (FDataTableEditorUtils::DuplicateRow(this, SourceRow, NewRow) !=NULL)
	{
		return true;
	}
	return false;
}

bool UPCapDataTable::AddTableRow(FName NewRow)
{
	if(FDataTableEditorUtils::AddRow(this, NewRow) == nullptr)
	{
		return false;
	}
	return true;
}

bool UPCapDataTable::InsertTableRow(FName SelectedRow, FName NewRow, bool bAbove)
{
	ERowInsertionPosition Position = ERowInsertionPosition::Above;
	if(!bAbove)
	{
		Position = ERowInsertionPosition::Below;
	}

	if(FDataTableEditorUtils::AddRowAboveOrBelowSelection(this, SelectedRow, NewRow, Position)!=NULL)
	{
		return true;
	}
	return false;
}

bool UPCapDataTable::UpdateTableRow(const FName& RowName, const FTableRowBase& RowData)
{
	check(0); // Never called directly — Blueprint VM dispatches to execUpdateTableRow
	return false;
}

DEFINE_FUNCTION(UPCapDataTable::execUpdateTableRow)
{
	P_GET_PROPERTY(FNameProperty, RowName);

	Stack.MostRecentPropertyAddress   = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.StepCompiledIn<FStructProperty>(nullptr);
	const uint8* const           RowData    = Stack.MostRecentPropertyAddress;
	const FStructProperty* const StructProp = CastField<FStructProperty>(Stack.MostRecentProperty);

	P_FINISH;

	bool bSuccess = false;

	if (RowName.IsNone())
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AccessViolation,
			NSLOCTEXT("PCapDataTable", "UpdateTableRow_InvalidRowName",
			          "UpdateTableRow: RowName must not be None."));
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
	else if (!StructProp || !RowData)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AccessViolation,
			NSLOCTEXT("PCapDataTable", "UpdateTableRow_MissingData",
			          "UpdateTableRow: RowData could not be resolved."));
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
	else
	{
		const UScriptStruct* const RowType   = StructProp->Struct;
		const UScriptStruct* const TableType = P_THIS->GetRowStruct();

		if (RowType != TableType) // The given row type must exactly match this table's RowStruct type. 
		{
			FBlueprintExceptionInfo ExceptionInfo(
				EBlueprintExceptionType::AccessViolation,
				NSLOCTEXT("PCapDataTable", "UpdateTableRow_WrongStruct",
				          "UpdateTableRow: RowData struct type does not match the DataTable's row struct."));
			FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
		}
		else
		{
			P_NATIVE_BEGIN;
			P_THIS->Modify();

			if (uint8* ExistingRow = P_THIS->FindRowUnchecked(RowName))
			{
				P_THIS->RowStruct->CopyScriptStruct(ExistingRow, RowData);
				P_THIS->HandleDataTableChanged(RowName);
				bSuccess = true;
				UE_LOG(LogPCap, Display, TEXT("Update table row %s in table %s"), *RowName.ToString(), *RowType->GetFName().ToString());
			}
			else
			{
				P_THIS->AddRow(RowName, RowData, RowType); //We do not call HandleDataTableChanged explicitly here as it is handled by AddRow on the parent class
				bSuccess = true;
				UE_LOG(LogPCap, Display, TEXT("Added table row %s in table %s"), *RowName.ToString(), *RowType->GetFName().ToString());
			}
			P_NATIVE_END;
		}
	}

	*(bool*)RESULT_PARAM = bSuccess;
}

