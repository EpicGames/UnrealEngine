// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DataTable.h"
#include "PCapDataTable.generated.h"

/**
 * 
 */
UCLASS()
class PERFORMANCECAPTUREWORKFLOW_API UPCapDataTable : public UDataTable
{
	GENERATED_BODY()
	
public:
	UPCapDataTable();
	virtual ~UPCapDataTable() override;

	DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnDatatableModified);
	UPROPERTY(BlueprintAssignable, Transient, Category="Performance Capture|Database")
	FOnDatatableModified OnDatatableModified;

	/** Emit a callback each time the datatable is modified*/
	void DataTableModified() const;

	/**
	 * Remove a given row from the datatable.
	 * @param RowName The row name to remove.
	 * @return bool Whether operation was successful.
	 */
	UFUNCTION(BlueprintCallable, Category="Performance Capture|Database")
	bool RemoveTableRow(FName RowName);

	/**
	 * Duplicate a given new row to the datatable.
	 * @param SourceRow The row to duplicate.
	 * @param NewRow The unique name for the new row.
	 * @return bool Whether duplication was successful.
	 */
	UFUNCTION(BlueprintCallable, Category="Performance Capture|Database")
	bool DuplicateTableRow(FName SourceRow, FName NewRow);

	/**
	 * Add a given new row to the datatable.
	 * @param NewRow The unique name for the new row.
	 * @return bool Whether operation was successful.
	 */
	UFUNCTION(BlueprintCallable, Category="Performance Capture|Database")
	bool AddTableRow(FName NewRow);
	
	/**
	 * Insert a given new row in to the datatable, above or below the selected row.
	 * @param SelectedRow The row to insert at.
	 * @param NewRow The unique name for the new row.
	 * @param bAbove Insert Above or Below the SelectedRow.
	 * @return bool Whether inserting new row was successful.
	 */
	UFUNCTION(BlueprintCallable, Category="Performance Capture|Database")
	bool InsertTableRow(FName SelectedRow, FName NewRow, bool bAbove);

	/**
	 * Update an existing row in-place (preserving row order), or add it if it does not exist.
	 * Unlike AddDataTableRow, this does not remove and re-insert the row, so order is preserved.
	 * Only compatiable with the natively declared PCapDataTable structs.
	 * @param RowName The name of the row to update or add.
	 * @param RowData The struct data to write into the row.
	 * @return true if the operation succeeded (row was updated or created); false if there was an error.
	 */
	UFUNCTION(BlueprintCallable, CustomThunk, Category="Performance Capture|Database",
		meta=(AutoCreateRefTerm="RowName", CustomStructureParam="RowData"))
	bool UpdateTableRow(const FName& RowName, const FTableRowBase& RowData);
	DECLARE_FUNCTION(execUpdateTableRow);
};
