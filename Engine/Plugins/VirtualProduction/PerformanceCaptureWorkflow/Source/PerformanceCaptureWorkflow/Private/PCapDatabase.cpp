// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCapDatabase.h"
#include "Algo/Transform.h"
#include "CapturePerformer.h"
#include "CaptureCharacter.h"
#include "PCapPropComponent.h"
#include "PCapSettings.h"
#include "PerformanceCapture.h"
#include "Misc/Guid.h"


FPCapRecordBase::FPCapRecordBase()
	: UID(FGuid::NewGuid())
{
}

FPCapRecordBase::~FPCapRecordBase()
{
}

void FPCapRecordBase::OnDataTableChanged(const UDataTable* InDataTable, const FName InRowName)
{
	FTableRowBase::OnDataTableChanged(InDataTable, InRowName);
}

void FPCapRecordBase::OnPostDataImport(const UDataTable* InDataTable, const FName InRowName, TArray<FString>& OutCollectedImportProblems)
{
	//TODO Handle not over-writing UID if a user imports
	FTableRowBase::OnPostDataImport(InDataTable, InRowName, OutCollectedImportProblems);
}

UPCapDataAsset::UPCapDataAsset()
{
	CreateGuid();
}

void UPCapDataAsset::CreateGuid()
{
	if (!AssetUID.IsValid())
	{
		AssetUID = FGuid::NewGuid();
	}
}

void UPCapDataAsset::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	Super::PostDuplicate(DuplicateMode);

	AssetUID = FGuid::NewGuid();
}

/** Performer DataAsset constructor */
UPCapPerformerDataAsset::UPCapPerformerDataAsset()
	: UPCapDataAsset()
	, PerformerActorClass(ACapturePerformer::StaticClass())
	, PerformerColor(1.0f, 1.0f, 1.0f, 1.0f)

{
}

/** Character DataAsset constructor */
UPCapCharacterDataAsset::UPCapCharacterDataAsset()
	: UPCapDataAsset()
	, CaptureCharacterClass(ACaptureCharacter::StaticClass())
{
}

TArray<UClass*> UPCapCharacterDataAsset::GetAllowedCaptureCharacterActorClasses()
{
	UPerformanceCaptureSettings* Settings = UPerformanceCaptureSettings::GetPerformanceCaptureSettings();
	TArray<UClass*> AllowedClasses;
	AllowedClasses.Reserve(Settings->AllowedCaptureCharacterActorClasses.Num());

	Algo::TransformIf(Settings->AllowedCaptureCharacterActorClasses, AllowedClasses,
		[](const TSoftClassPtr<ASkeletalMeshActor>& InSoftClass) { return !InSoftClass.IsNull(); },
		[](const TSoftClassPtr<ASkeletalMeshActor>& InSoftClass) -> UClass* { return InSoftClass.LoadSynchronous(); }
	);
	return AllowedClasses;
}

TArray<UClass*> UPCapCharacterDataAsset::GetDisallowedCaptureCharacterActorClasses()
{
	UPerformanceCaptureSettings* Settings = UPerformanceCaptureSettings::GetPerformanceCaptureSettings();
	TArray<UClass*> DisallowedClasses;
	DisallowedClasses.Reserve(Settings->DisallowedCaptureCharacterActorClasses.Num());

	Algo::TransformIf(Settings->DisallowedCaptureCharacterActorClasses, DisallowedClasses,
		[](const TSoftClassPtr<ASkeletalMeshActor>& InSoftClass) { return !InSoftClass.IsNull(); },
		[](const TSoftClassPtr<ASkeletalMeshActor>& InSoftClass) -> UClass* { return InSoftClass.LoadSynchronous(); }
	);
	return DisallowedClasses;
}

UPCapPropDataAsset::UPCapPropDataAsset()
	: UPCapDataAsset()
	, PropComponentClass(UPCapPropComponent::StaticClass())
{
}

/** Prop DataAsset PostEditChange */
void UPCapPropDataAsset::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if(PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPCapPropDataAsset, bClearEditConditions))
	{
		bClearEditConditions = false;
		bCanSetStaticMesh = true;
		bCanSetSkeletalMesh = true;
		bCanSetCustomPropClass =true;
	}
	if(PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPCapPropDataAsset, PropStaticMesh))
	{
		FString PrintMe = PropStaticMesh.ToString();
		bool ValidThing = PropStaticMesh.IsNull();
		UE_LOGF(LogPCap, Display, "Static Mesh is %ls", ( ValidThing ? TEXT("true") : TEXT("false") ));
		
		if(!PropStaticMesh.IsNull())
		{
			bCanSetSkeletalMesh = false;
			bCanSetCustomPropClass = false;
		}
		else if(PropStaticMesh.IsNull())
		{
			bCanSetStaticMesh = true;
			bCanSetSkeletalMesh = true;
			bCanSetCustomPropClass =true;
		}
	}
	if(PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPCapPropDataAsset, PropSkeletalMesh))
	{
		if(!PropSkeletalMesh.IsNull())
		{
			bCanSetStaticMesh = false;
			bCanSetCustomPropClass = false;
		}
		else if(PropSkeletalMesh.IsNull())
		{
			bCanSetStaticMesh = true;
			bCanSetSkeletalMesh = true;
			bCanSetCustomPropClass =true;
		}
	}
	if(PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPCapPropDataAsset, CustomPropClass))
	{
		if(!CustomPropClass.IsNull())
		{
			bCanSetStaticMesh = false;
			bCanSetSkeletalMesh = false;
		}
		else if(CustomPropClass.IsNull())
		{
			bCanSetStaticMesh = true;
			bCanSetSkeletalMesh = true;
			bCanSetCustomPropClass =true;
		}
	}
}

void UPCapPropDataAsset::PostLoad()
{
	Super::PostLoad();
	ValidateEditConditions();
}

void UPCapPropDataAsset::ValidateEditConditions()
{
	if(!CustomPropClass.IsNull())
    {
    	bCanSetStaticMesh = false;
    	bCanSetSkeletalMesh = false;
    }
	if(!PropStaticMesh.IsNull())
	{
		bCanSetSkeletalMesh = false;
		bCanSetCustomPropClass = false;
	}
	if(!PropSkeletalMesh.IsNull())
	{
		bCanSetStaticMesh = false;
		bCanSetCustomPropClass = false;
	}	
}
