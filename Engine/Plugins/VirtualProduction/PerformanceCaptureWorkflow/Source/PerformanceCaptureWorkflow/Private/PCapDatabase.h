// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CapturePerformer.h"
#include "CaptureCharacter.h"
#include "LevelSequence.h"
#include "LiveLinkTypes.h"
#include "PCapDataTable.h"
#include "Engine/DataTable.h"
#include "Engine/DataAsset.h"
#include "Misc/DateTime.h"
#include "PCapDatabase.generated.h"

class UPCapPropDataAsset;
class UPCapCharacterDataAsset;
class UPCapPerformerDataAsset;
class UPCapSessionTemplate;
class UPCapDataTable;
class ULevelSequence;
class UIKRigDefinition;
class UIKRetargeter;
class ULiveLinkInstance;
class UDataLayerAsset;
class USkeletalMesh;
class UPCapPropComponent;

/// Database structs ///

USTRUCT()
struct FPCapRecordBase : public FTableRowBase
{
	GENERATED_BODY()

	FPCapRecordBase();
	virtual ~FPCapRecordBase() override;

	//Only called when you import or re-import data. TODO Make sure the UIDs are not overwritten

	virtual void OnDataTableChanged(const UDataTable* InDataTable, const FName InRowName) override;

	virtual void OnPostDataImport(const UDataTable* InDataTable, const FName InRowName, TArray<FString>& OutCollectedImportProblems) override;
	
	/** GUID of the production record struct */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category="Performance Capture", meta=(IgnoreForMemberInitializationTest))
	FGuid UID;

	/** Bool to control if a record should be considered archived so the UI can hide it from view. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Performance Capture")
	bool bIsArchived = false;
	
};

/** Struct to hold the record of a production. */
USTRUCT(BlueprintType)
struct FPCapProductionRecord : public FPCapRecordBase
{
	GENERATED_BODY()

public:
	/** Name of the production. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Performance Capture Production")
	FName ProductionName;

	/** Notes on the production. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Performance Capture Production")
	FString ProductionNotes;
};

/** Take status enumerator. Thumbs Up, Down and Neutral. */
UENUM(BlueprintType)
enum class EPCapTakeStatus : uint8
{
	ThumbsUp,
	ThumbsDown,
	Neutral
};

/** Struct to hold the record of a recorded take. */
USTRUCT(BlueprintType)
struct FPCapTakeRecord : public FPCapRecordBase
{
	GENERATED_BODY()
	
public:

	/** Temporary Flag. Transient, so will not be saved. */
	UPROPERTY(Transient, BlueprintReadWrite, Category="Flag")
	bool bFlag = false;
	
	/** The date and time the sequence was recorded. This is the local machine
	 * UTC at the moment recording is finalised
	 */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category="Performance Capture Take")
	FDateTime DateTimeCreated;
	
	/** The level sequence recorded. */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category="Performance Capture Take")
	TSoftObjectPtr<ULevelSequence> RecordedTake;
	
	/** Seconds Duration. */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category="Performance Capture Take")
	float TakeDurationSeconds = 0;

	/** Recorded Framerate. */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category="Performance Capture Take")
	FFrameRate Framerate = FFrameRate(30, 1);
	
	/** HHMMSSFF Duration. */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category="Performance Capture Take")
	FTimecode TakeDurationTimecode;

	/** Start Timecode. */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category="Performance Capture Take")
	FTimecode StartTimecode;

	/** End Timecode. */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category="Performance Capture Take")
	FTimecode EndTimecode;

	/** Transform of the stage root when this recording was made. */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category="Performance Capture Take")
	FTransform MocapStageRootTransform;

	/** Status of Take - Thumbs Up, Thumbs Down or None. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Performance Capture Take")
	EPCapTakeStatus TakeStatus = EPCapTakeStatus::Neutral;

	/** 5-Star Rating. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Performance Capture Take", meta=(ClampMin=0, ClampMax=5))
	int32 Rating = 0;
	
	/** Does this recording LiveLinkSource tracks. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Performance Capture Take")
	bool bContainsLiveLinkSources = false;

	/** Has this take been processed for plotting */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Performance Capture Take")
	bool bLiveLinkPlotted = false;
	
	/**
	 * Has the animation recorded for this take been replaced with an external asset import.
	 * This option will be positive if you use the import and upgrade workflow
	 * bringing fbx recordings from your motion capture software in and
	 * replacing the live recording.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Performance Capture Take")
	bool bExternallyReplaced = false;

	/** GUID for the session used during the recording of this take. */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category="Performance Capture Take")
	FGuid SessionUID;
};

/** Struct to hold a datatable record of a session. */
USTRUCT(BlueprintType)
struct FPCapSessionRecord : public FPCapRecordBase
{
	GENERATED_BODY()
	
public:
	
	/** Name for this session */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category="Performance Capture Session")
	FName SessionName;

	/** Date and time session record was created. */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category="Performance Capture Session")
	FDateTime SessionDateTime;

	/** Notes for this session record. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Performance Capture Session")
	FString SessionNotes;

	/** Name of Production this Session belongs to. */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category="Performance Capture Session")
	FName ProductionName;

	/** GUID of the Production record associated with this session record. */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category="Performance Capture Session")
	FGuid ProductionUID;

	/** The token generated name of the session. */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category="Performance Capture Session", meta = (DisplayName = "Session OutputName"))
	FString SessionOutputName;

	/** The content browser path to this session record's data. */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category="Performance Capture Session", meta = (DisplayName = "Session Folder", ContentDir))
	FString SessionPath;

	/** The content browser path to this session record's performer data. */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite , Category="Performance Capture Session", meta = (DisplayName = "Performer Folder", ContentDir))
	FString PerformerPath;
	
	/** The content browser path to this session record's character data. */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category="Performance Capture Session", meta = (DisplayName = "Character Folder", ContentDir))
	FString CharacterPath;
	
	/** The content browser path to this session record's ingest data. */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category="Performance Capture Session", meta = (DisplayName = "Ingest Folder", ContentDir))
	FString IngestPath;

	/** The content browser path to this session record's recorded takes data. */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category="Performance Capture Session", meta = (DisplayName = "Takes Folder", ContentDir))
	FString TakesPath;

	/** The content browser path to this session record's prop data. */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category="Performance Capture Session", meta = (DisplayName = "Prop Folder", ContentDir))
	FString PropPath;

	/** The content browser path to this session record's scene data. */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category="Performance Capture Session", meta = (DisplayName = "Scene Folder", ContentDir))
	FString ScenePath;

	/** The content browser path to this session record's common data. */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category="Performance Capture Session", meta = (DisplayName = "Common Folder", ContentDir))
	FString CommonPath;

	/** Array of paths to folders of additional data. These folders can be defined in by the Session Template dataasset. */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category="Performance Capture Session", meta = (DisplayName = "Additional Folders", ContentDir))
	TArray<FString> AdditionalFolders;

	/** Reference to the Takes datatable created for and used by this session record. */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category="Performance Capture Session")
	TSoftObjectPtr<UPCapDataTable> TakesDataTable;

	/** Array of soft-refs to Performers spawned during this session. */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category="Performance Capture Session")
	TArray<TSoftObjectPtr<UPCapPerformerDataAsset>> Performers;

	/** Array of soft-refs to Characters spawned during this session. */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category="Performance Capture Session")
	TArray<TSoftObjectPtr<UPCapCharacterDataAsset>> Characters;

	/** Array of soft-refs to Props spawned during this session. */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category="Performance Capture Session")
	TArray<TSoftObjectPtr<UPCapPropDataAsset>> Props;

	/** Reference to a locked Session Template. This will be generated on session creation. */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category="Performance Capture Session")
	TSoftObjectPtr<UPCapSessionTemplate> SessionTemplate;

	/** Bool to determine if this session record while in a level streaming or world partition level. */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category="Performance Capture Session")
	bool bIsWorldPartition = false;

	/** Sub-level created for this session, if using a persistent level of the Level streaming type. */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category="Performance Capture Session")
	TSoftObjectPtr<UWorld> SubLevel;

	/** Datalayer created for this session record, if using a persistent level of the World Partition type.  */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category="Performance Capture Session")
	TSoftObjectPtr<UDataLayerAsset> SessionDataLayer;

	/** The slates datatable to use in Mocap Recorder for this session. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Performance Capture Session", meta = (RequiredAssetDataTags = "RowStructure=/Script/PerformanceCaptureWorkflow.PCapSlateRecord"))
	TSoftObjectPtr<UPCapDataTable> SessionSlateTable;
};

/** Slate Status. Can be Incomplete, Complete, Skip.*/
UENUM(BlueprintType)
enum class EPCapSlateStatus : uint8
{
	Incomplete	UMETA(Tooltip="Slate is incomplete"),
	Complete	UMETA(Tooltip="Slate is marked complete"),
	Skip		UMETA(Tooltip="Slate is marked to skip")
};

/** Slate struct. A slate record is a what the user wishes call a take and provides prior to recording */
USTRUCT(BlueprintType)
struct FPCapSlateRecord : public FPCapRecordBase
{                                                                                                                                                                                                                                                                                                                                                                                                                                 
	GENERATED_BODY()

	/** String to provide Name of Slate.  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Performance Capture Slate", meta=(DisplayName="Slate"))
	FString Slate;
	
	/** Note on slate. This will be passed to the Mocap Recorder when using the Mocap Manager panel. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Performance Capture Slate", meta=(DisplayName="Note", MultiLine=true))
	FString SlateNote;

	/** Status of Slate, defined by the status enum. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Performance Capture Slate", meta=(DisplayName="Status"))
	EPCapSlateStatus SlateStatus = EPCapSlateStatus::Incomplete;

	/** UID of the session this Slate is used in. */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category="Performance Capture Slate")
	FGuid SessionUID;
};

///// End Database Structs //////////

/////////////////////////////////////
/// Begin Data Asset Definitions/////
/////////////////////////////////////

/// Data Assets are for saving collections of assets that should/need to be gathered together

/** Performance Capture DataAsset. Contains references to assets used for Performance Capture Workflows*/
UCLASS(Abstract, MinimalAPI)
class UPCapDataAsset : public UPrimaryDataAsset
{
	GENERATED_BODY()
	
public:
	/** Constructor*/
	UPCapDataAsset();

	void CreateGuid();

	virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;

	UFUNCTION(BlueprintGetter, Category="Performance Capture|Admin")
	FGuid GetAssetUID() const {return AssetUID;}

private:

	/** Guid for disambiguating actors spawned by the data assets. Only editable from BP. */
	UPROPERTY(BlueprintGetter="GetAssetUID", Category="Performance Capture|Admin", meta=(IgnoreForMemberInitializationTest))
	FGuid AssetUID;
};

/** Mocap Performer DataAsset. This class is intended to track and encapsulate the properties and assets that make up a performer */
UCLASS(Blueprintable, BlueprintType)
class UPCapPerformerDataAsset : public UPCapDataAsset
{
	GENERATED_BODY()

public:
	/** Constructor*/
	UPCapPerformerDataAsset();

	/** LiveLink Subject for the body of this Performer */
	UPROPERTY(BlueprintReadWrite, EditDefaultsOnly, Category="Performance Capture Performer")
	FName PerformerName;
	
	/** LiveLink Subject for this Performer. */
	UPROPERTY(BlueprintReadWrite, EditDefaultsOnly, Category="Performance Capture Performer")
	FLiveLinkSubjectName LiveLinkSubject;
	
	/** Performer Actor class to use for this performer. */
	UPROPERTY(BlueprintReadWrite, EditDefaultsOnly, NoClear, Category="Performance Capture Performer")
	TSoftClassPtr<ACapturePerformer> PerformerActorClass;
	
	/** Color for this performer. */
	UPROPERTY(BlueprintReadWrite, EditDefaultsOnly, Category="Performance Capture Performer")
	FLinearColor PerformerColor;
	
	/** Performer Base Skeleton. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Performance Capture Performer")
	TSoftObjectPtr<USkeletalMesh> BaseSkeletalMesh;
	
	/** Performer Mesh. This must be created in a T or A Pose performer from a LiveLink pose*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Performance Capture Performer")
	TSoftObjectPtr<USkeletalMesh> PerformerProportionedMesh;

	/** Performer IKRig. */
	UPROPERTY(BlueprintReadWrite, EditDefaultsOnly, Category="Performance Capture Performer")
	TSoftObjectPtr<UIKRigDefinition> IKRig;
};

/** Mocap Character Data asset. This class is intended to track and encapsulate the properties and assets that make up a character */
UCLASS(Blueprintable, BlueprintType)
class UPCapCharacterDataAsset : public UPCapDataAsset
{
	GENERATED_BODY()
	
public:

	// Capture Character Constructor */
	UPCapCharacterDataAsset();

	/** Capture Character Name. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Performance Capture Character", meta=(Tooltip="Optional Name for Character. If blank DataAsset name will be used"))
	FName CharacterName;

	/** Source Performer Asset for this Character. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Performance Capture Character")
	TSoftObjectPtr<UPCapPerformerDataAsset> SourcePerformerAsset;

	/** Character actor Class */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Performance Capture Character",
		meta = (GetAllowedClasses = "GetAllowedCaptureCharacterActorClasses", GetDisallowedClasses = "GetDisallowedCaptureCharacterActorClasses"))
	TSoftClassPtr<ASkeletalMeshActor>  CaptureCharacterClass;

	/** Main Skeletal Mesh Asset. This will be the root component of any characters spawned.
	 * A character will not be spawned if this asset reference is null or not valid.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Performance Capture Character")
	TSoftObjectPtr<USkeletalMesh> SkeletalMesh;
	
	/** IKRig Asset. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Performance Capture Character")
	TSoftObjectPtr<UIKRigDefinition> IKRig;

	/** Retarget Asset. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Performance Capture Character")
	TSoftObjectPtr<UIKRetargeter> Retargeter;

#if WITH_EDITORONLY_DATA
	/** Array of additional skeleltal mesh assets that will be spawned and parented to the root skeleteal mesh asset*/
	UE_DEPRECATED(5.7, "AdditionalMeshes is no longer used. Create a SkeletalMeshActor class with the required meshes and components instead.")
	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage = "Create a SkeletalMeshActor class with the required meshes and components instead."))
	TArray<TSoftObjectPtr<USkeletalMesh>> AdditionalMeshes_DEPRECATED;
#endif // WITH_EDITORONLY_DATA

private:
	UFUNCTION()
	static TArray<UClass*> GetAllowedCaptureCharacterActorClasses();

	UFUNCTION()
	static TArray<UClass*> GetDisallowedCaptureCharacterActorClasses();
};

/**  Prop DataAsset*/
UCLASS(Blueprintable, BlueprintType)
class UPCapPropDataAsset : public UPCapDataAsset
{
	GENERATED_BODY()
	
public:
	// Prop DataAsset Constructor */
	UPCapPropDataAsset();
	
	/** Prop Name. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Performance Capture Prop", meta=(Tooltip="Optional Name for Prop. If blank LiveLinkSubject name will be used"))
	FName PropName;
	
	/** Prop's LiveLinkSubject. */
	UPROPERTY(BlueprintReadWrite, EditDefaultsOnly, Category="Performance Capture Prop")
	FLiveLinkSubjectName LiveLinkSubject;

	/** Static Mesh Offset Transform **/
	UPROPERTY(BlueprintReadWrite, EditDefaultsOnly, Category="Performance Capture Prop")
	FTransform PropOffsetTransform;

	/** Prop Component - select a customized prop component if required, otherwise base C++ class will be used*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Performance Capture Prop", meta=(DisplayName="Prop Component Class"), NoClear)
	TSoftClassPtr<UPCapPropComponent> PropComponentClass;
	
	/** Prop Static - if this is left blank the mocap static mesh will be used*/
	UPROPERTY(BlueprintReadWrite, EditDefaultsOnly, Category="Performance Capture Prop", meta=(DisplayName="Static Mesh", EditCondition="bCanSetStaticMesh", HideEditConditionToggle))
	TSoftObjectPtr<UStaticMesh> PropStaticMesh;
	
	/** Prop SkelMesh - if this is left blank the mocap static mesh will be used*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Performance Capture Prop", meta=(DisplayName="Skeletal Mesh", EditCondition="bCanSetSkeletalMesh", HideEditConditionToggle))
	TSoftObjectPtr<USkeletalMesh> PropSkeletalMesh;

	/** Custom Class - if you want your prop to be created from a custom BP actor, set the class here*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Performance Capture Prop", meta=(DisplayName="Custom Prop Class", EditCondition="bCanSetCustomPropClass", HideEditConditionToggle))
	TSoftClassPtr<AActor> CustomPropClass;

	/** Bool to control if this prop will be set to bHiddenInGame=True when spawned. Useful for proxy objects you don't want to see in game view.*/
	UPROPERTY(BlueprintReadWrite, EditDefaultsOnly, Category="Performance Capture Prop", meta=(Tooltip="Choose True for props you want to hide in 'Game' view e.g. Proxies or integrators"))
	bool bHiddenInGame = false;

	/** bool to control edit condition of the Static and Skeletal mesh properties */
	UPROPERTY(EditDefaultsOnly, Transient, Category="Performance Capture Prop", AdvancedDisplay, meta=(DisplayName="Reset Read-only Options", Tooltip="Reset all mesh options (removes read-only status on fields)"))
	bool bClearEditConditions = false;

	//~ UObject interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	virtual void PostLoad() override;
#endif
	//~ End UObject interface

private:

	UPROPERTY()
	bool bCanSetStaticMesh = true;
	UPROPERTY()
	bool bCanSetSkeletalMesh = true;
	UPROPERTY()
	bool bCanSetCustomPropClass =true;

	void ValidateEditConditions();
};

/** Editor-only class to enable the user to determine if creating database records should be done using intenral datatable assets or connect
 * to some external database.
 */
//TODO currently a stub and not yet implemented.
#if WITH_EDITOR
UCLASS(Blueprintable, Abstract, meta = (ShowWorldContextPin))
class UPerformanceCaptureDatabaseHelper : public UObject
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintImplementableEvent)
	void CreateRecord();

	UFUNCTION(BlueprintImplementableEvent)
	void DeleteRecord();
};

#endif
