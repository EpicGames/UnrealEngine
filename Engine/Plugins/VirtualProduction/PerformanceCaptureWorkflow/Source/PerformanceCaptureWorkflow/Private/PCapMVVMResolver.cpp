// Copyright Epic Games, Inc. All Rights Reserved.


#include "PCapMVVMResolver.h"
#include "Engine/Engine.h" //Is this allowed?
#include "PCapSubsystem.h"
#include "PCapSettings.h"
#include "Types/MVVMViewModelCollection.h"

#if WITH_EDITOR

UObject* UPCapMVVMResolver::CreateInstance(const UClass* ExpectedType, const UUserWidget* UserWidget, const UMVVMView* View) const
{
	const UPerformanceCaptureSettings* Settings = UPerformanceCaptureSettings::GetPerformanceCaptureSettings();
	const UPerformanceCaptureSubsystem* Subsystem =  GEngine->GetEngineSubsystem<UPerformanceCaptureSubsystem>();
	const UMVVMViewModelCollectionObject* Collection = Subsystem->GetViewModelCollection();

	//Check the viewmodel class is valid and force load it if it's not already there.
	if(UClass* ViewModelClass = Settings->ViewModelClass.LoadSynchronous())
	{
		FMVVMViewModelContext Context;
		Context.ContextClass = ViewModelClass;
		Context.ContextName = "PerformanceCaptureWorkflow";
		
		return Collection->FindViewModelInstance(Context);
	}
	return Super::CreateInstance(ExpectedType, UserWidget, View);
}
#endif