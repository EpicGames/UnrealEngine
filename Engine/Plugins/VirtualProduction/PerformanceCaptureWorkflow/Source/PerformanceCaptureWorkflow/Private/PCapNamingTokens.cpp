// Copyright Epic Games, Inc. All Rights Reserved.


#include "PCapNamingTokens.h"

#include "PCapSessionTemplate.h"
#include "PerformanceCapture.h"

#define LOCTEXT_NAMESPACE "PCapNamingTokens"

UPCapNamingTokens::UPCapNamingTokens()
{
	Namespace = GetPCapNamespace();
}

UPCapNamingTokens::~UPCapNamingTokens()
{
}

void UPCapNamingTokens::OnCreateDefaultTokens(TArray<FNamingTokenData>& Tokens)
{
	Super::OnCreateDefaultTokens(Tokens);

	Tokens.Add( {
		TEXT("session"),
		LOCTEXT("PCapTokenSession", "Session"),
		FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]
		{
			if (Context && Context->SessionTemplate)
			{
				return FText::FromString(Context->SessionTemplate->SessionName);
			}

			UE_LOGF(LogPCap, Verbose, "Attempted to evaluate token but no valid context is available.");
			
			return FText::GetEmpty();
		})
	});
	
	Tokens.Add( {
		TEXT("sessionToken"),
		LOCTEXT("PCapTokenSessionToken", "SessionToken"),
		FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]
		{
			if (Context && Context->SessionTemplate)
			{
				return FText::FromString(Context->SessionTemplate->SessionToken.Output);
			}

			UE_LOGF(LogPCap, Verbose, "Attempted to evaluate token but no valid context is available.");
			
			return FText::GetEmpty();
		})
	});

	Tokens.Add( {
		TEXT("production"),
		LOCTEXT("PCapTokenProduction", "Production"),
		FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]
		{
			if (Context)
			{
				return FText::FromString(Context->SessionTemplate->ProductionName);
				
			}

			UE_LOGF(LogPCap, Verbose, "Attempted to evaluate token but no valid context is available.");

			return FText::GetEmpty();
		})
	});
	
	Tokens.Add( {
		TEXT("pcapRootFolder"),
		LOCTEXT("PCapTokenRootFolder", "RootFolder"),
		FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]
		{
			if (Context)
			{
				return FText::FromString(Context->SessionTemplate->TemplateRootFolder.Path);
			}

			UE_LOGF(LogPCap, Verbose, "Attempted to evaluate token but no valid context is available.");

			return FText::GetEmpty();
		})
	});
		
	Tokens.Add( {
		TEXT("sessionFolder"),
		LOCTEXT("PCapTokenSessionFolder", "SessionFolder"),
		FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]
		{
			if (Context)
			{
				return FText::FromString(Context->SessionTemplate->SessionFolder.FolderPathOutput);
			}
			UE_LOGF(LogPCap, Verbose, "Attempted to evaluate token but no valid context is available.");

			return FText::GetEmpty();
			
		})
	});
}

void UPCapNamingTokens::OnPreEvaluate_Implementation(const FNamingTokensEvaluationData& InEvaluationData)
{
	Super::OnPreEvaluate_Implementation(InEvaluationData);

	UPCapNamingTokensContext* MatchingContext = nullptr;
	InEvaluationData.Contexts.FindItemByClass<UPCapNamingTokensContext>(&MatchingContext);
	Context = MatchingContext;
}

void UPCapNamingTokens::OnPostEvaluate_Implementation()
{
	Super::OnPostEvaluate_Implementation();
	Context = nullptr;
}

#undef LOCTEXT_NAMESPACE