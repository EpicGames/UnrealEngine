// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GlobalNamingTokens.h"
#include "PCapNamingTokens.generated.h"

class UPCapSessionTemplate;

/**
 * Naming tokens class for Pcap workflow
 */
UCLASS(NotBlueprintable)
class PERFORMANCECAPTUREWORKFLOW_API UPCapNamingTokens : public UNamingTokens
{
	GENERATED_BODY()

public:

	UPCapNamingTokens();
	virtual ~UPCapNamingTokens() override;
	
	static FString GetPCapNamespace()
	{
		return TEXT("pcap");
	}

protected:

	// ~Begin UNamingTokens
	virtual void OnCreateDefaultTokens(TArray<FNamingTokenData>& Tokens) override;
	virtual void OnPreEvaluate_Implementation(const FNamingTokensEvaluationData& InEvaluationData) override;
	virtual void OnPostEvaluate_Implementation() override;
	// ~End UNamingTokens

private:

	UPROPERTY(Transient)
	TObjectPtr<UPCapNamingTokensContext> Context;
};

/** Context Object that references the current session template and allows it to be passed to the naming token functions */
UCLASS(MinimalAPI)

class UPCapNamingTokensContext : public UObject
{
	GENERATED_BODY()
	
public:
	
	//The current template in use. This is so the session template can pass a reference of itself. 
	UPROPERTY(Transient)
	TObjectPtr<const UPCapSessionTemplate> SessionTemplate;
};