// Copyright Epic Games, Inc. All Rights Reserved.


#include "PCapSessionTemplate.h"
#include "NamingTokensEngineSubsystem.h"
#include "PCapBPFunctionLibrary.h"
#include "Engine/Engine.h"

UPCapSessionTemplate::UPCapSessionTemplate(const FObjectInitializer& ObjectInitializer)
{
	bIsEditable = true;
	NamingTokensContext = NewObject<UPCapNamingTokensContext>();
	SubsequenceDirectory.Template = FString("{takeName}_Subscenes"); //Matched to the default in take recorder.
}

void UPCapSessionTemplate::PostLoad()
{
	Super::PostLoad();
	
}

void UPCapSessionTemplate::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property != nullptr)
	{
		UpdateAllFields();
	}
}

void UPCapSessionTemplate::UpdateAllFields()
{

	if (bIsEditable) //If this asset is not "locked" then do not update any of the strings. 
	{
		//Make sure all the tokens are updated each time a property is edited
		SessionToken		= UpdateStringTemplate(SessionToken);
	
		SessionFolder		= UpdateFolderPathTemplate(SessionFolder);
		CommonFolder		= UpdateFolderPathTemplate(CommonFolder);
		CharacterFolder		= UpdateFolderPathTemplate(CharacterFolder);
		IngestFolder		= UpdateFolderPathTemplate(IngestFolder);
		PerformerFolder		= UpdateFolderPathTemplate(PerformerFolder);
		PropFolder			= UpdateFolderPathTemplate(PropFolder);
		SceneFolder			= UpdateFolderPathTemplate(SceneFolder);
		TakeFolder			= UpdateFolderPathTemplate(TakeFolder);

		for (TPair<FName, FPCapTokenisedFolderPath>& Pair : AdditionalFolders)
		{
			Pair.Value = UpdateFolderPathTemplate(Pair.Value);
		}

	
		TakeSaveName		= UpdateStringTemplate(TakeSaveName);
		AnimationTrackName	= UpdateStringTemplate(AnimationTrackName);
		AnimationAssetName	= UpdateStringTemplate(AnimationAssetName);
		AnimationSubDirectory= UpdateStringTemplate(AnimationSubDirectory);
		SubsequenceDirectory= UpdateStringTemplate(SubsequenceDirectory);
		AudioSourceName		= UpdateStringTemplate(AudioSourceName);
		AudioTrackName		= UpdateStringTemplate(AudioTrackName);
		AudioAssetName		= UpdateStringTemplate(AudioAssetName);
		AudioSubDirectory	= UpdateStringTemplate(AudioSubDirectory);
	}
}

FPCapTokenisedString UPCapSessionTemplate::UpdateStringTemplate(FPCapTokenisedString& InTokenisedTemplate)
{
	FPCapTokenisedString Template = InTokenisedTemplate;
	
	if (!NamingTokensContext)
	{
		NamingTokensContext = NewObject<UPCapNamingTokensContext>();
	}

	if (bIsEditable && NamingTokensContext)
	{
		NamingTokensContext->SessionTemplate = this;

		FNamingTokenFilterArgs NamingTokenFilters;
		NamingTokenFilters.AdditionalNamespacesToInclude = { FString("pcap") , FString("tr")};

		FNamingTokenResultData NamingTokenResultData = GEngine->GetEngineSubsystem<UNamingTokensEngineSubsystem>()->EvaluateTokenText(FText::FromString(Template.Template), NamingTokenFilters, {NamingTokensContext});
		
		Template.Output = UPerformanceCaptureBPFunctionLibrary::SanitizeFileString(NamingTokenResultData.EvaluatedText.ToString());
	}

	return Template;
	
}

FPCapTokenisedFolderPath UPCapSessionTemplate::UpdateFolderPathTemplate(FPCapTokenisedFolderPath& InFolderPathTokenisedTemplate)
{
	FPCapTokenisedFolderPath Template = InFolderPathTokenisedTemplate;
	if (!NamingTokensContext)
	{
		NamingTokensContext = NewObject<UPCapNamingTokensContext>();
	}

	if (bIsEditable && NamingTokensContext)
	{
		NamingTokensContext->SessionTemplate = this;

		FNamingTokenFilterArgs NamingTokenFilters;
		NamingTokenFilters.AdditionalNamespacesToInclude = { FString("pcap") };

		FNamingTokenResultData NamingTokenResultData = GEngine->GetEngineSubsystem<UNamingTokensEngineSubsystem>()->EvaluateTokenText(FText::FromString(Template.FolderPathTemplate), NamingTokenFilters, {NamingTokensContext});
		
		Template.FolderPathOutput = UPerformanceCaptureBPFunctionLibrary::SanitizePathString(NamingTokenResultData.EvaluatedText.ToString());
	}

	return Template;
}
