// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PCapDatabase.h"
#include "PCapNamingTokens.h"
#include "PCapSessionTemplate.generated.h"

/**
 * Struct for handling conversion of token entry template into a validated folder path
 */

USTRUCT(BlueprintType)
struct FPCapTokenisedString
{
	GENERATED_BODY()

	FPCapTokenisedString() = default;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Folder Template")
	FString Template;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="Folder Template")
	FString Output;
	
};

USTRUCT(BlueprintType)
struct FPCapTokenisedFolderPath
{
	GENERATED_BODY()

	FPCapTokenisedFolderPath() = default;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Folder Template")
	FString FolderPathTemplate;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="Folder Template")
	FString FolderPathOutput;
	
};

/**
 * Data asset for defining all the folders that will be created for a session
 * Can be duplicated and locked to ensure all tokenized strings become serialized static data.
 */

UCLASS(Blueprintable, BlueprintType)
class UPCapSessionTemplate : public UPCapDataAsset
{
	GENERATED_BODY()

	UPCapSessionTemplate(const FObjectInitializer& ObjectInitializer);

public:

	/*
	* Core session fields
	*/

	UPROPERTY(EditAnywhere, BlueprintSetter = SetRootFolder, BlueprintGetter = GetRootFolder, Category="Template", meta = (ContentDir, DisplayName = "Root Folder", EditCondition="bIsEditable"))
	FDirectoryPath TemplateRootFolder;

	/*
	 *These properties are hidden from BP access because we want to force the use of the Blueprint Getter and Setter so modifications only follow that path.
	 */

	virtual void PostLoad() override;
	
	/** String for production name. Will be sanitized for illegal filesystem characters*/
	UPROPERTY(EditAnywhere, Category="Template", BlueprintGetter = GetProductionName, BlueprintSetter = SetProductionName,  meta = (DisplayName = "Production Name", EditCondition="bIsEditable"))
	FString ProductionName;

	/** String for session name. Will be sanitized for illegal filesystem character */
	UPROPERTY(EditAnywhere, Category="Template", BlueprintGetter = GetSessionName, BlueprintSetter = SetSessionName, meta = (DisplayName = "Session Name", EditCondition="bIsEditable"))
	FString SessionName;

	/** The final name of the session, evaluated from the given tokens and static strings */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Template", meta = (DisplayName = "Session Token", EditCondition="bIsEditable", tooltip="The final name of our session, including any tokens you include in it's construction. You can use tokens to build this string, including the Session Name above"))
	FPCapTokenisedString SessionToken;

	/**
	 * Set the RootFolder.
	 */
	UFUNCTION(BlueprintSetter)
	void SetRootFolder(FDirectoryPath NewRootPath)
	{
		TemplateRootFolder = NewRootPath;
		UpdateAllFields();
	}

	/**
	 * Get the RootFolder.
	 * @return TemplateRootFolder path.
	 */
	UFUNCTION(BlueprintGetter)
	FDirectoryPath GetRootFolder()
	{
		return TemplateRootFolder;
	}
	
	/**
	 * Get the Session Name.
	 * @return SessionName string.
	 */
	UFUNCTION(BlueprintGetter)
	FString GetSessionName()
	{
		return SessionName;
	}

	/**
	 * Set the session name
	 * @param NewSessionName New session name string. This will be sanitized for illegal filesystem characters.
	 */
	UFUNCTION(BlueprintSetter)
	void SetSessionName(FString NewSessionName)
	{
		SessionName = NewSessionName;
		UpdateAllFields();
	}

	/**
	 * Get the Production Name string.
	 * @return ProductionName  
	 */
	UFUNCTION(BlueprintGetter)
	FString GetProductionName()
	{
		return ProductionName;
	}

	/**
	 * Set New Production Name string. This will be sanitized for illegal filesystem characters 
	 * @param NewProductionName 
	 */
	UFUNCTION(BlueprintSetter)
	void SetProductionName(FString NewProductionName)
	{
		ProductionName = NewProductionName;
		UpdateAllFields();
	}
	

	/*------------------------------------------------------------------------------
	Folder template definitions - note these hard-coded and "opinionated" about how users will work. 
	------------------------------------------------------------------------------*/

	
	/*
	Session folder template
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Folders", DisplayName = "Session Folder", meta=(EditCondition="bIsEditable"))
	FPCapTokenisedFolderPath SessionFolder;

	/**Common folder template**/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Folders|Session Subfolders", meta = (DisplayName = "Common Folder", EditCondition="bIsEditable"))
	FPCapTokenisedFolderPath CommonFolder;

	/**Character folder template*/	
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Folders|Session Subfolders", meta = (DisplayName = "Character Folder", EditCondition="bIsEditable"))
	FPCapTokenisedFolderPath CharacterFolder;
	
	/**Ingest data folder template*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Folders|Session Subfolders", meta = (DisplayName = "Ingest Folder", EditCondition="bIsEditable"))
	FPCapTokenisedFolderPath IngestFolder;

	/**Performer folder template*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Folders|Session Subfolders", meta = (DisplayName = "Performer Folder", EditCondition="bIsEditable"))
	FPCapTokenisedFolderPath PerformerFolder;

	/**Prop folder template*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Folders|Session Subfolders", meta = (DisplayName = "Prop Folder", EditCondition="bIsEditable"))
	FPCapTokenisedFolderPath PropFolder;

	/**Scene data folder template*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Folders|Session Subfolders", meta = (DisplayName = "Scene Folder", EditCondition="bIsEditable"))
	FPCapTokenisedFolderPath SceneFolder;

	/**Take Recorder folder template*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Folders|Session Subfolders", meta = (DisplayName = "Takes Folder", EditCondition="bIsEditable"))
	FPCapTokenisedFolderPath TakeFolder;

	/** Additional folders. Create any additional folders you want under the session folder. You can use the Map key to label what each folder is for*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Folders|Session Subfolders", meta=(DisplayAfter="Additional Folders", EditCondition="bIsEditable", ToolTip="Use the additional folders for any extra data you want contain in your session - eg audio and facial capture data"))
	TMap<FName, FPCapTokenisedFolderPath> AdditionalFolders;
	
	// Take Record related fields */

	/** Bool to control if timecode is recorded or not. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Take Recorder", meta = (DisplayName = "Record Timecode", EditCondition="bIsEditable"))
	bool bRecordTimecode = true;

	/** Bool to control if all actors should be recorded to possessable or spawnable. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Take Recorder", meta = (DisplayName = "Record to Possessable", EditCondition="bIsEditable"))
	bool bRecordPossessable = false;

	/** Bool to control if recorded sequences should start at the current timecode value. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Take Recorder", meta = (DisplayName = "Start at Current Timecode", EditCondition="bIsEditable"))
	bool bStartAtCurrentTimecode = true;
	
	/** The clock source to use with recordings. Recommended is "Timecode". */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Take Recorder", meta = (DisplayName = "Recording Clock Source", EditCondition="bIsEditable"))
	EUpdateClockSource RecordingClockSource = EUpdateClockSource::Timecode;

	/** Bool to control if each recording source is placed into a subscene at the start of recording. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Take Recorder", meta = (DisplayName = "Record to Subsequences", EditCondition="bIsEditable"))
	bool bRecordSubscenes = true;

	/** Take name token. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Take Recorder", DisplayName = "Take Save Name", meta=(EditCondition="bIsEditable",  tooltip ="This be used by take recorder as the name of the recorded level sequence and the folder for any sub-scenes and assets"))
	FPCapTokenisedString TakeSaveName;

	/** Animation track name token. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Take Recorder|Animation", DisplayName = "Animation Track Name", meta=(EditCondition="bIsEditable"))
	FPCapTokenisedString AnimationTrackName;
	
	/** Animation Asset name token. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Take Recorder|Animation", DisplayName = "Animation Asset Name", meta=(EditCondition="bIsEditable"))
	FPCapTokenisedString AnimationAssetName;

	/**Animation subdirectory token. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Take Recorder|Animation", DisplayName = "Animation Sub Directory", meta=(EditCondition="bIsEditable"))
	FPCapTokenisedString AnimationSubDirectory;

	/** Subsequence directory name token. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Take Recorder|Animation", DisplayName = "Subsequence Directory", meta=(EditCondition="bIsEditable", tooltip ="This affects the directory created for subsequences. It cannot be set per actor, only globally across the whole recording."))
	FPCapTokenisedString SubsequenceDirectory;
	
	/** Audio source name token. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Take Recorder|Audio", DisplayName = "Audio Source Name", meta=(EditCondition="bIsEditable"))
	FPCapTokenisedString AudioSourceName;

	/** Audio track name token. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Take Recorder|Audio", DisplayName = "Audio Track Name", meta=(EditCondition="bIsEditable"))
	FPCapTokenisedString AudioTrackName;

	/** Audio asset name token. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Take Recorder|Audio", DisplayName = "Audio Asset Name", meta=(EditCondition="bIsEditable"))
	FPCapTokenisedString AudioAssetName;

	/** Audio subdirectory token. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Take Recorder|Audio", DisplayName = "Audio Sub Directory", meta=(EditCondition="bIsEditable"))
	FPCapTokenisedString AudioSubDirectory;

	/*
	 * Read only fields to show the user what tokens are available to them in the pcap, global, and take recorder namespaces
	 */
	UPROPERTY(VisibleAnywhere, Category="Tokens", meta= (displayname = "PCap Tokens", tooltip="Use these  tokens in your template strings. The namespace for these tokens is 'pcap.'")) // this is for display purposes only, not to be used in code.
	FString PCapTokens =
	"{session} - Name for your Performance Capture session\n"
	"{production} - Name of Performance Capture production\n"
	"{sessionToken} - Output value of the Session Token field\n"
	"{pcapRootFolder} - Root folder for all Performance Capture productions and sessions\n"
	"{sessionFolder} - folder path for Performance Capture session"
	/*"\n"
	"\n"
	"\n"*/;

	UPROPERTY(VisibleAnywhere, Category="Tokens", meta= (displayname = "Global Tokens", tooltip="Use these  tokens in your template strings.")) // this is for display purposes only, not to be used in code.
	FString GlobalTokens =
	"{yyyy} - Year (4 digit)\n"
	"{yy} - Year (2 digit)\n"
	"{Mmm} - 3-character Month (Pascal Case)\n"
	"{MMM} - 3-character Month (UPPERCASE)\n"
	"{mmm} - 3-character Month (lowercase)\n"
	"{mm} - Month (2 digit)\n"
	"{Ddd} - 3-character Day (Pascal Case)\n"
	"{DDD} - 3-character Day (UPPERCASE)\n"
	"{ddd} - 3-character Day (lowercase)\n"
	"{dd} - Day (2 digit)\n"
	"{ampm} - am or pm (lowercase)\n"
	"{AMPM} - AM or PM (UPPERCASE)\n"
	"{12h} - Hour (12)\n"
	"{24h} - Hour (24)\n"
	"{min} - Minute\n"
	"{sec} - Second\n"
	"{ms} - Millisecond";

	UPROPERTY(VisibleAnywhere, Category="Tokens", meta= (displayname = "Take Recorder Tokens", tooltip="Use these  tokens in your template strings. The namespace for these tokens is 'pcap.'")) // this is for display purposes only, not to be used in code.
	FString TakeRecorderTokens =
	"{day}\n"
	"{month}\n"
	"{year}\n"
	"{hour}\n"
	"{minute}\n"
	"{second}\n"
	"{take}\n"
	"{slate}\n"
	"{map}\n"
	"{actor}\n"
	"{channel} - audio channel. Only available for audio recording tracks";
	
protected:

	/*
	 * Use this to effectively "lock" the asset and prevent any dynamic tokens from being
	 * Behind the scenes, this prevents PostEditUpdates from modifying data such that makes all
	 * the strings and folder paths become static
	 */
	/** Bool to control edit condition on the members of this asset. Prevents user from editing after a session has been created.*/
	UPROPERTY(BlueprintReadWrite, VisibleAnywhere, Category="Admin")
	bool bIsEditable;
	
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	/**
	 * Update all members and ensure illegal characters are removed. 
	 */
	UFUNCTION(BlueprintCallable, Category="Performance Capture|Session Template")
	void UpdateAllFields();

	/**
	 * Update string token from the given template.
	 * @param TokenisedTemplate 
	 * @return Updated string token struct.
	 */
	FPCapTokenisedString UpdateStringTemplate(FPCapTokenisedString& TokenisedTemplate);

	/**
	 * Upate a token for a folder path.
	 * @param FolderPathTokenisedTemplate 
	 * @return Updated folder path struct.
	 */
	FPCapTokenisedFolderPath UpdateFolderPathTemplate(FPCapTokenisedFolderPath& FolderPathTokenisedTemplate);
	
private:

	UPROPERTY()
	TObjectPtr<UPCapNamingTokensContext> NamingTokensContext;
};

