// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCapSettings.h"
#include "ISettingsModule.h"
#include "PCapDatabase.h"
#include "PCapDataTable.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "UPerformanceCaptureSettings"

namespace PerformanceCaptureSettings
{
	
}

UPerformanceCaptureSettings* UPerformanceCaptureSettings::GetPerformanceCaptureSettings()
{
	return GetMutableDefault<UPerformanceCaptureSettings>();
}

/** Summon the Project settings for the Performance Capture module*/
void UPerformanceCaptureSettings::ShowPerformanceCaptureProjectSettings()
{
	ISettingsModule& SettingsModule = FModuleManager::LoadModuleChecked<ISettingsModule>("Settings");
	SettingsModule.ShowViewer("Project", "Plugins","PerformanceCapture");
}

#if WITH_EDITOR
void UPerformanceCaptureSettings::SetSessionTable(TSoftObjectPtr<UPCapDataTable> NewDataTable)
{
	SessionTable = NewDataTable;

	FProperty* SessionTableProperty = FindFieldChecked<FProperty>(UPerformanceCaptureSettings::StaticClass(),GET_MEMBER_NAME_CHECKED(UPerformanceCaptureSettings, SessionTable));
	FPropertyChangedEvent  PCapSessionTableChangedEvent = FPropertyChangedEvent(SessionTableProperty, EPropertyChangeType::ValueSet);
	PostEditChangeProperty(PCapSessionTableChangedEvent);
	TryUpdateDefaultConfigFile();
}

void UPerformanceCaptureSettings::SetProductionTable(TSoftObjectPtr<UPCapDataTable> NewDataTable)
{
	ProductionTable = NewDataTable;
	FProperty* ProductionTableProperty = FindFieldChecked<FProperty>(UPerformanceCaptureSettings::StaticClass(),GET_MEMBER_NAME_CHECKED(UPerformanceCaptureSettings, ProductionTable));
	FPropertyChangedEvent  PCapProductionTableChangedEvent = FPropertyChangedEvent(ProductionTableProperty, EPropertyChangeType::ValueSet);
	PostEditChangeProperty(PCapProductionTableChangedEvent);
	TryUpdateDefaultConfigFile();
}

void UPerformanceCaptureSettings::SetDefaultSessionTemplate(TSoftObjectPtr<UPCapSessionTemplate> NewSessionTemplate)
{
	DefaultSessionTemplate = NewSessionTemplate;
	FProperty* DefaultSessionTemplateProperty = FindFieldChecked<FProperty>(UPerformanceCaptureSettings::StaticClass(), GET_MEMBER_NAME_CHECKED(UPerformanceCaptureSettings, DefaultSessionTemplate));
	FPropertyChangedEvent  PCapDefaultSessionTemplateChangedEvent = FPropertyChangedEvent(DefaultSessionTemplateProperty, EPropertyChangeType::ValueSet);
	PostEditChangeProperty(PCapDefaultSessionTemplateChangedEvent);
	TryUpdateDefaultConfigFile();
}


void UPerformanceCaptureSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if(PropertyChangedEvent.Property !=nullptr)
	{
		OnPCapSettingsChanged.Broadcast();
	}
}

#endif

#undef LOCTEXT_NAMESPACE
