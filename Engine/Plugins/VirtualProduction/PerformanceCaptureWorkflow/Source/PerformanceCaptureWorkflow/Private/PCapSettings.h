// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/SkeletalMeshActor.h"
#include "CoreMinimal.h"
#include "EditorUtilityWidgetBlueprint.h"
#include "PCapStageRoot.h"
#include "PCapSubsystem.h"
#include "MVVMViewModelBase.h"
#include "UObject/Object.h"
#include "PCapSettings.generated.h"

class UPCapWorkflowCustomization;
class APerformanceCaptureStageRoot;
class UPCapSessionTemplate;
class UDataTable;
class UPCapDataTable;
class USkeletalMesh;

/*
 * Per Project Settings for Performance Capture.
 */
UCLASS(Config=PerformanceCaptureWorkflow, DefaultConfig, DisplayName="Performance Capture")

class PERFORMANCECAPTUREWORKFLOW_API UPerformanceCaptureSettings : public UObject
{
	GENERATED_BODY()
public:
	/**
	 * Get the Performance Capture settings uobject
	 * @return Settings Object.
	 */
	UFUNCTION(BlueprintPure, Category="Performance Capture|Settings")
	static UPerformanceCaptureSettings* GetPerformanceCaptureSettings();

	/**
	 * Open the project settings panel and show the Performance Capture section.
	 */
	UFUNCTION(BlueprintCallable, Category = "Performance Capture|Settings")
	void ShowPerformanceCaptureProjectSettings();
	
	/** Stage Root Actor to spawn for Performance Capture. Class must derive from APerformanceCaptureStageRoot. */
	UPROPERTY(BlueprintReadOnly, EditDefaultsOnly, Config, NoClear, Category="Actors", DisplayName= "Stage Root Actor Class" ,meta = (MetaClass = "/Script/PerformanceCaptureWorkflow.PerformanceCaptureStageRoot"))
	TSoftClassPtr<APerformanceCaptureStageRoot> StageRoot;

	
	/** Base skeletal mesh for Mocap performers. */
	UPROPERTY(BlueprintReadOnly, EditDefaultsOnly, Config, NoClear, Category="Actors", DisplayName= "Default Performer Mesh")
	TSoftObjectPtr<USkeletalMesh> DefaultPerformerSkelMesh;

	/** Skeletal Mesh Actor classes that are explicitly to be used as Characters with Mocap Manager. These skeletal mesh actors must contain a retarget component*/
	UPROPERTY(BlueprintReadOnly, EditDefaultsOnly, Config, NoClear, Category = "Actors")
	TArray<TSoftClassPtr<ASkeletalMeshActor>> AllowedCaptureCharacterActorClasses;

	/** Skeletal Mesh Actor classes that are explicitly disallowed for use as Characters with Mocap Manager. */
	UPROPERTY(BlueprintReadOnly, EditDefaultsOnly, Config, NoClear, Category = "Actors")
	TArray<TSoftClassPtr<ASkeletalMeshActor>> DisallowedCaptureCharacterActorClasses;

	/** Blueprint Viewmodel class. If you change class you will need to restart the editor to pick up the change. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Config, NoClear, Category="UI", meta = (MetaClass = "/Script/ModelViewViewModel.MVVMViewModelBase",ConfigRestartRequired=true))
	TSoftClassPtr<UMVVMViewModelBase> ViewModelClass;

	/** Editor Utility Widget class that will be used for the Mocap Manager UI*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Config, Category="UI", meta = (MetaClass = "/Script/Blutility.EditorUtilityWidgetBlueprint"))
	TSoftObjectPtr<UEditorUtilityWidgetBlueprint> MocapManagerUI;

	/** Default Workflow UI panels to use in the Mocap Manager panel. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Config, Category="UI", DisplayName = "Default Workflows", meta = ( tooltip="This Data Asset defines the UI panels you wish to have available in Mocap Manager"))
	TSoftObjectPtr<UPCapWorkflowCustomization> MocapManagerDefaultWorkflow;

	/** 
	 * Editor Utility Widget class that will be used for each of the phase UI panels. Note this is not editable in the settings.
	 * It can only be modified in the DefaultPerformanceCaptureWorkflow.ini
	 */
	UPROPERTY(BlueprintReadOnly, Config, NoClear, Category="UI", meta = (MetaClass = "/Script/Blutility.EditorUtilityWidgetBlueprint"))
	TSoftClassPtr<UEditorUtilityWidget> WorkflowPhaseUIBaseClass;

	/** Default session template to use when starting the Mocap Manager panel. */
	UPROPERTY(BlueprintReadOnly, Config, EditDefaultsOnly, Category="Database", DisplayName = "Default Session Template", meta = ( tooltip="This Asset defines the folder structure and settings used in a performance capture session"))
	TSoftObjectPtr<UPCapSessionTemplate> DefaultSessionTemplate;
	
	/** Blueprint helper class for making database calls. If you change class you will need to restart the editor to pick up the change*/
	UPROPERTY(EditAnywhere, Config, NoClear, Category="Database", DisplayName = "Database Helper Class", meta=(ConfigRestartRequired=true))
	TSoftClassPtr<UPerformanceCaptureDatabaseHelper> DatabaseHelperClass;

	/** Pointer to the datatable Mocap Manager will use for recording all session data. */
	UPROPERTY(BlueprintReadOnly, Config, EditDefaultsOnly, Category = "Database|Internal Database Tables", DisplayName = "Session DataTable", meta = (RequiredAssetDataTags = "RowStructure=/Script/PerformanceCaptureWorkflow.PCapSessionRecord"))
	TSoftObjectPtr<UPCapDataTable> SessionTable;

	/** Pointer to the datatable Mocap Manager will use for recording production data. */
	UPROPERTY(BlueprintReadOnly, Config, EditDefaultsOnly, Category="Database|Internal Database Tables", DisplayName = "Production DataTable", meta = (RequiredAssetDataTags = "RowStructure=/Script/PerformanceCaptureWorkflow.PCapProductionRecord"))
	TSoftObjectPtr<UPCapDataTable> ProductionTable;

	/**
	 * Set the session table in Performance Capture settings. 
	 * @param NewDataTable Datatable ref. Must be of the FPCapSession struct type.
	 */
	UFUNCTION(BlueprintCallable, Category="Performance Capture|Settings")
	void SetSessionTable(TSoftObjectPtr<UPCapDataTable> NewDataTable);

	/**
	 * Set the production table in Performance Capture Settings.
	 * @param NewDataTable Datatable ref. Must be of the FPCapProduction struct type.
	 */
	UFUNCTION(BlueprintCallable, Category="Performance Capture|Settings")
	void SetProductionTable(TSoftObjectPtr<UPCapDataTable> NewDataTable);

	/**
	 * Set the default session template in Performance Capture Settings.
	 * @param NewSessionTemplate Session Template data asset ref.
	 */
	UFUNCTION(BlueprintCallable, Category="Performance Capture|Settings")
	void SetDefaultSessionTemplate(TSoftObjectPtr<UPCapSessionTemplate> NewSessionTemplate); 

#if WITH_EDITOR
	
	DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnPCapSettingsChanged);

	/**
	 * Multicast delegate called whenever the Performance Capture settings object is modified.
	 */
	UPROPERTY(BlueprintAssignable, Category="Performance Capture|Settings")
	FOnPCapSettingsChanged OnPCapSettingsChanged;
	
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
};

/**
 * Utility object for choosing a content browser path in a message dialog.
 */
UCLASS(Blueprintable)
class UPCapDialogObject : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=nocategory, meta=(ContentDir, DisplayName="PCap Path", ToolTip="Choose a folder for your Performance Capture data"))
	FDirectoryPath Path;
};
