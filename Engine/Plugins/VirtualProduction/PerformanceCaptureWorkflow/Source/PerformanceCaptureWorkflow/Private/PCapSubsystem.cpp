// Copyright Epic Games, Inc. All Rights Reserved.


#include "PCapSubsystem.h"
#include "PCapDatabase.h"
#include "PCapSettings.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Editor/TransBuffer.h"
#include "Types/MVVMViewModelCollection.h"
#include "Misc/CoreDelegates.h"
#include "UObject/Package.h"
#include "Editor.h"
#include "ILiveLinkClient.h"
#include "Features/IModularFeatures.h"
#include "Subsystems/AssetEditorSubsystem.h"


void UPerformanceCaptureSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
	
	ViewModelCollection = NewObject<UMVVMViewModelCollectionObject>(this);

	EngineInitCompleteDelegate = FCoreDelegates::OnFEngineLoopInitComplete.AddUObject(this,&UPerformanceCaptureSubsystem::OnEngineInitComplete);
}

void UPerformanceCaptureSubsystem::OnEngineInitComplete()

{
	EngineInitCompleteDelegate.Reset();

	InitiateDatabaseHelper();
	InitiateViewModelCollection();

	const UPerformanceCaptureSettings* Settings = UPerformanceCaptureSettings::GetPerformanceCaptureSettings();

	/** Force load the stage class*/
	UClass* StageRootClass =  Settings->StageRoot.LoadSynchronous();

	/** Force load the UI class - this is needed so the UI class is available when the user's layout is created*/
	UEditorUtilityWidgetBlueprint* MocapManagerUI =  Settings->MocapManagerUI.LoadSynchronous();

	/** Get the Asset Registry and bind to the Remove, Duplicate and Add asset events*/
	AssetRegistry = &FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();

	AssetRegistry->OnAssetRemoved().AddUObject(this, &UPerformanceCaptureSubsystem::OnAssetRemoved);
	AssetRegistry->OnAssetRenamed().AddUObject(this, &UPerformanceCaptureSubsystem::OnAssetRenamed);
	AssetRegistry->OnAssetAdded().AddUObject(this, &UPerformanceCaptureSubsystem::OnAssetAdded);
	FCoreUObjectDelegates::OnObjectPropertyChanged.AddUObject(this, &UPerformanceCaptureSubsystem::OnObjectPropertyChanged);

#if WITH_EDITOR
	if (GEditor)
	{
		if (UTransBuffer* TransactionBuffer = Cast<UTransBuffer>(GEditor->Trans))
		{
			TransactionBuffer->OnUndo().AddUObject(this, &UPerformanceCaptureSubsystem::OnEditorUndo);
			TransactionBuffer->OnRedo().AddUObject(this, &UPerformanceCaptureSubsystem::OnEditorRedo);
		}

		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OnAssetOpenedInEditor().AddUObject(this, &UPerformanceCaptureSubsystem::OnAssetOpened);
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OnAssetClosedInEditor().AddUObject(this, &UPerformanceCaptureSubsystem::OnAssetClosed);
	}
#endif
	
// Bind to LiveLink Subject changed delegate
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		ILiveLinkClient& LiveLinkClient = IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
		LiveLinkClient.OnLiveLinkSubjectStateChanged().AddUObject(this, &UPerformanceCaptureSubsystem::OnLiveLinkSubjectUpdated);
		LiveLinkClient.OnLiveLinkSubjectAdded().AddUObject(this, &UPerformanceCaptureSubsystem::OnLiveLinkSubjectAdded);
		LiveLinkClient.OnLiveLinkSubjectRemoved().AddUObject(this, &UPerformanceCaptureSubsystem::OnLiveLinkSubjectRemoved);
		LiveLinkClient.OnLiveLinkSubjectEnabledChanged().AddUObject(this, &UPerformanceCaptureSubsystem::OnLiveLinkSubjectEnableChanged);
	}
	
// Bind to changes in the Timecode and  Custom Timestep providers.
	if(GEngine)
	{
		GEngine->OnCustomTimeStepChanged().AddUObject(this, &UPerformanceCaptureSubsystem::OnCustomTimestepProviderChanged);
		GEngine->OnTimecodeProviderChanged().AddUObject(this, &UPerformanceCaptureSubsystem::OnTimecodeProviderChanged);
	}
}

void UPerformanceCaptureSubsystem::InitiateDatabaseHelper()
{
	/**Instantiate the database helper class*/
	const UPerformanceCaptureSettings* Settings = UPerformanceCaptureSettings::GetPerformanceCaptureSettings();
	if(UClass* HelperClass =  Settings->DatabaseHelperClass.LoadSynchronous())
	{
		DatabaseHelper = NewObject<UPerformanceCaptureDatabaseHelper>(GetTransientPackage(), HelperClass);
	}
}

void UPerformanceCaptureSubsystem::InitiateViewModelCollection()
{
	/**Instantiate the viewmodel class*/
	const UPerformanceCaptureSettings* Settings = UPerformanceCaptureSettings::GetPerformanceCaptureSettings();
	if(UClass* ViewModelClass = Settings->ViewModelClass.LoadSynchronous())
	{
		PerformanceCaptureViewModel = NewObject<UMVVMViewModelBase>(GetTransientPackage(), ViewModelClass);
		FMVVMViewModelContext Context;
		Context.ContextClass = ViewModelClass;
		Context.ContextName = "PerformanceCaptureWorkflow";
		ViewModelCollection->AddViewModelInstance(Context, PerformanceCaptureViewModel);
	}
}

void UPerformanceCaptureSubsystem::OnAssetRemoved(const FAssetData& InAssetData) const
{
	OnPCapAssetRemoved.Broadcast(InAssetData);
}

void UPerformanceCaptureSubsystem::OnAssetRenamed(const FAssetData& InAssetData, const FString& OldName) const
{
	OnPCapAssetRenamed.Broadcast(InAssetData, OldName);
}

void UPerformanceCaptureSubsystem::OnAssetAdded(const FAssetData& InAssetData) const
{
	OnPCapAssetAdded.Broadcast(InAssetData);
}

void UPerformanceCaptureSubsystem::OnObjectPropertyChanged(UObject* Asset, FPropertyChangedEvent& PropertyChangedEvent)
{
	AActor* Actor = Cast<AActor>(Asset);
	if(Actor)
	{
		OnPCapActorModified.Broadcast(Actor);
	}
}

void UPerformanceCaptureSubsystem::OnEditorUndo(const FTransactionContext& TransactionContext, bool bSucceeded) const
{
	if(bSucceeded)
	{
		OnPCapEditorUndo.Broadcast(true);
	}
}

void UPerformanceCaptureSubsystem::OnEditorRedo(const FTransactionContext& TransactionContext, bool bSucceeded) const
{
	if(bSucceeded)
	{
		OnPCapEditorRedo.Broadcast(true);
	}
}

void UPerformanceCaptureSubsystem::OnAssetOpened(UObject* Object, IAssetEditorInstance* Instance) const
{
	OnPCapAssetEditorOpen.Broadcast(Object);
}

void UPerformanceCaptureSubsystem::OnAssetClosed(UObject* Object, IAssetEditorInstance* Instance) const
{
	OnPCapAssetEditorClose.Broadcast(Object);
}

void UPerformanceCaptureSubsystem::OnLiveLinkSubjectUpdated(FLiveLinkSubjectKey Subject, ELiveLinkSubjectState State) const
{
	OnPCapLiveLinkSubjectUpdate.Broadcast(Subject, State);
}

void UPerformanceCaptureSubsystem::OnLiveLinkSubjectRemoved(FLiveLinkSubjectKey Subject) const
{
	OnPCapLiveLinkSubjectRemoved.Broadcast(Subject);
}

void UPerformanceCaptureSubsystem::OnLiveLinkSubjectAdded(FLiveLinkSubjectKey Subject) const
{
	OnPCapLiveLinkSubjectAdded.Broadcast(Subject);
}

void UPerformanceCaptureSubsystem::OnLiveLinkSubjectEnableChanged(FLiveLinkSubjectKey Subject, bool NewEnabled) const
{
	OnPCapLiveLinkSubjectEnableChanged.Broadcast(Subject, NewEnabled);
}

void UPerformanceCaptureSubsystem::OnTimecodeProviderChanged() const
{
	OnPCapTimecodeProviderChanged.Broadcast();
}

void UPerformanceCaptureSubsystem::OnCustomTimestepProviderChanged() const
{
	OnPCapCustomTimestepProviderChanged.Broadcast();
}

