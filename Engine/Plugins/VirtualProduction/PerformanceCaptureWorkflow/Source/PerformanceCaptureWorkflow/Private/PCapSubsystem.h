// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LiveLinkTypes.h"
#include "Subsystems/EngineSubsystem.h"
#include "MVVMViewModelBase.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/AssetData.h"
#include "UObject/Object.h"

#include "PCapSubsystem.generated.h"

class UPerformanceCaptureDatabaseHelper;
class UMVVMViewModelCollectionObject;
class IAssetEditorInstance;

#if WITH_EDITOR
struct FTransactionContext;
#endif

/**
 * 
 */
UCLASS(Category="Performance Capture")
class PERFORMANCECAPTUREWORKFLOW_API UPerformanceCaptureSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

public:
	// ~Begin UEditorSubsystem
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	// ~End UEditorSubsystem

	/**
	 * Get the Performance Capture Database helper.
	 * @return Databasehelper object
	 */
	UFUNCTION(BlueprintCallable, Category="Performance Capture|Database")
	UPerformanceCaptureDatabaseHelper* GetDatabaseHelper() const
	{
		return DatabaseHelper;
	}

	/**
	 * Get the Performance Capture Viewmodel collection.
	 * @return Performance Capture Viewmodel collection.
	 */
	UFUNCTION(BlueprintCallable, Category="Performance Capture|Viewmodel")
	UMVVMViewModelCollectionObject* GetViewModelCollection() const
	{
		return ViewModelCollection;
	}

#if WITH_EDITORONLY_DATA
	
	/**
	* Multicast delegate that will execute when an asset is removed from the asset registry. Returns the FAssetData of the removed asset.
	*/
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnPCapAssetRemoved, FAssetData, DeletedAsset);
	/** Editor Only*/
	UPROPERTY(BlueprintAssignable, Category="Performance Capture|Assets")
	FOnPCapAssetRemoved OnPCapAssetRemoved;

	/**
	* Multicast delegate that will execute when an asset is renamed to the asset registry. Returns the FAssetData of the renamed asset.
	*/
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnPCapAssetRenamed, FAssetData, RenamedAsset, FString, OldName);
	/** Editor Only*/
	UPROPERTY(BlueprintAssignable, Category="Performance Capture|Assets")
	FOnPCapAssetRenamed OnPCapAssetRenamed;

	/**
	* Multicast delegate that will execute when a new asset is added to the asset registry. Returns the FAssetData of new asset.
	*/
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnPCapAssetAdded, FAssetData, NewAsset);
	/** Editor Only*/
	UPROPERTY(BlueprintAssignable, Category="Performance Capture|Assets")
	FOnPCapAssetAdded OnPCapAssetAdded;

	/**
	* Multicast delegate that will execute when an actor is modified in the Level Editor. Returns the actor modified.
	*/
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnPCapActorModified, AActor*, Actor);
	/** Editor Only*/
	UPROPERTY(BlueprintAssignable, Category="Performance Capture|Workflow")
	FOnPCapActorModified OnPCapActorModified;
	
	/**
	* Multicast delegate that will execute when the user calls Undo. Returns success boolean.
	*/
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnPCapEditorUndo, bool, bSuccess);
	/** Editor Only*/
	UPROPERTY(BlueprintAssignable, Category="Performance Capture|Workflow")
	FOnPCapEditorUndo OnPCapEditorUndo;

	/**
	* Multicast delegate that will execute when the user calls Redo. Returns success boolean.
	*/
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnPCapEditorRedo, bool, bSuccess);
	/** Editor Only*/
	UPROPERTY(BlueprintAssignable, Category="Performance Capture|Workflow")
	FOnPCapEditorRedo OnPCapEditorRedo;

	/**
	* Multicast delegate that will execute when an asset editor is opened. Returns the asset object that is opened. 
	*/
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnPCapAssetEditorOpen, UObject*, Asset);
	/** Editor Only*/
	UPROPERTY(BlueprintAssignable, Category="Performance Capture|Assets")
	FOnPCapAssetEditorOpen OnPCapAssetEditorOpen;

	/**
	* Multicast delegate that will execute when an asset editor is closed. Returns the asset object that is closed. 
	*/
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnPCapAssetEditorClose, UObject*, Asset);
	/** Editor Only*/
	UPROPERTY(BlueprintAssignable, Category="Performance Capture|Assets")
	FOnPCapAssetEditorClose OnPCapAssetEditorClose;

	/**
	* Multicast delegate that will execute when a Live Link subject's state is updated.
	*/
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnPCapLiveLinkSubjectUpdate, FLiveLinkSubjectKey, Subject, ELiveLinkSubjectState, State);
	/** Editor Only*/
	UPROPERTY(BlueprintAssignable, Category="Performance Capture|LiveLink")
	FOnPCapLiveLinkSubjectUpdate OnPCapLiveLinkSubjectUpdate;

	/**
	* Multicast delegate that will execute when a Live Link subject is added.
	*/
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnPCapLiveLinkSubjectAdded, FLiveLinkSubjectKey, Subject);
	/** Editor Only*/
	UPROPERTY(BlueprintAssignable, Category="Performance Capture|LiveLink")
	FOnPCapLiveLinkSubjectAdded OnPCapLiveLinkSubjectAdded;

	/**
	* Multicast delegate that will execute when a Live Link subject is removed.
	*/
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnPCapLiveLinkSubjectRemoved, FLiveLinkSubjectKey, Subject);
	/** Editor Only*/
	UPROPERTY(BlueprintAssignable, Category="Performance Capture|LiveLink")
	FOnPCapLiveLinkSubjectRemoved OnPCapLiveLinkSubjectRemoved;

	/**
	* Multicast delegate that will execute when a Live Link subject's bIsEnabled member is modified in the Live Link panel.
	*/
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnPCapLiveLinkSubjectEnableChanged, FLiveLinkSubjectKey, Subject, bool, NewEnabled);
	/** Editor Only*/
	UPROPERTY(BlueprintAssignable, Category="Performance Capture|LiveLink")
	FOnPCapLiveLinkSubjectEnableChanged OnPCapLiveLinkSubjectEnableChanged;
	
	/**
	* Multicast delegate that will execute when the engine's Timecode provider is updated.
	*/
	DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnPCapTimecodeProviderChanged);
	/** Editor Only*/
	UPROPERTY(BlueprintAssignable, Category="Performance Capture|LiveLink")
	FOnPCapTimecodeProviderChanged OnPCapTimecodeProviderChanged;
	
	/**
	* Multicast delegate that will execute when the engine's Timestep provider is updated.
	*/
	DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnPCapCustomTimestepProviderChanged);
	/** Editor Only*/
	UPROPERTY(BlueprintAssignable, Category="Performance Capture|LiveLink")
	FOnPCapCustomTimestepProviderChanged OnPCapCustomTimestepProviderChanged;


#endif

protected:
	void OnEngineInitComplete();

	void InitiateDatabaseHelper();

	void InitiateViewModelCollection();
	
	UPROPERTY(Transient)
	TObjectPtr<UPerformanceCaptureDatabaseHelper> DatabaseHelper;

	UPROPERTY(Transient)
	TObjectPtr<UMVVMViewModelBase> PerformanceCaptureViewModel;
	
	FDelegateHandle EngineInitCompleteDelegate;

	IAssetRegistry* AssetRegistry;

private:
	UPROPERTY(Transient)
	TObjectPtr<UMVVMViewModelCollectionObject> ViewModelCollection;

	void OnAssetRemoved(const FAssetData& InAssetData) const;

	void OnAssetRenamed(const FAssetData& InAssetData, const FString& NewName) const;
	
	void OnAssetAdded(const FAssetData& InAssetData) const;

	void OnObjectPropertyChanged(UObject* Asset, FPropertyChangedEvent& PropertyChangedEvent);
	
	void OnEditorUndo(const FTransactionContext& TransactionContext, bool Succeeded) const;

	void OnEditorRedo(const FTransactionContext& TransactionContext, bool Succeeded) const;

	void OnAssetOpened(UObject* Object, IAssetEditorInstance* Instance) const;
	
	void OnAssetClosed(UObject* Object, IAssetEditorInstance* Instance) const;

	void OnLiveLinkSubjectUpdated(FLiveLinkSubjectKey Subject, ELiveLinkSubjectState State) const;

	void OnLiveLinkSubjectAdded(FLiveLinkSubjectKey Subject) const;

	void OnLiveLinkSubjectRemoved(FLiveLinkSubjectKey Subject) const;

	void OnLiveLinkSubjectEnableChanged(FLiveLinkSubjectKey Subject, bool NewEnabled) const;
	
	void OnTimecodeProviderChanged() const;
	
	void OnCustomTimestepProviderChanged() const;
};

//Todo: add post edit change to re-int the viewmodel and database helper if the user changes these once the editor has started. 


