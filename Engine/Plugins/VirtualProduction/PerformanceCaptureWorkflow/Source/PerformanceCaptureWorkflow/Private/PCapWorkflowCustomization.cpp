// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCapWorkflowCustomization.h"
#include "PCapSettings.h"

TArray<UClass*> UPCapWorkflowCustomization::GetAllowedPanelWidgetClasses()
{
	UPerformanceCaptureSettings* Settings = UPerformanceCaptureSettings::GetPerformanceCaptureSettings();
	TArray<UClass*> AllowedClasses;
	AllowedClasses.Reserve(1);
	if (Settings != nullptr && !Settings->WorkflowPhaseUIBaseClass.IsNull())
	{
		AllowedClasses.Emplace(Settings->WorkflowPhaseUIBaseClass.LoadSynchronous());
	}
	return AllowedClasses;
}

#if WITH_EDITOR
void UPCapWorkflowCustomization::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	if(PropertyChangedEvent.Property != nullptr)
	{
		OnPCapCustomizationModified.Broadcast();
	}
}
#endif

