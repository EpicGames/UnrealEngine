// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorUtilityWidget.h"
#include "PCapDatabase.h"
#include "Engine/DataAsset.h"
#include "PCapWorkflowCustomization.generated.h"

class UEditorUtilityWidget;

USTRUCT(BlueprintType)
struct FMocapManagerPhase
{
	GENERATED_BODY()

	/**
	* Name to use for this phase. It should be unique among all the other phase names
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "MocapManager")
	FName PhaseName;
	
	/**
	 * Icon to use on the phase button on the Mocap Manager UI
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "MocapManager")
	TSoftObjectPtr<UTexture2D> Icon;

	/**
	 * Bool to determine if Mocap Manager should use this phase
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "MocapManager")
	bool bIsEnabled = true;

	/**
	 * UI Panel to use for this phase. Must be of class defined by WorkflowPhaseUIBaseClass in DefaultPerformanceCaptureWorkflow.ini 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "MocapManager", meta = (GetAllowedClasses = "/Script/PerformanceCaptureWorkflow.PCapWorkflowCustomization:GetAllowedPanelWidgetClasses"))
	TSoftClassPtr<UEditorUtilityWidget> WorkflowPhaseUIPanelClass;
	
};

/**
 * Data asset class to let users build a series of phases in Mocap Manager
 */
UCLASS(Blueprintable, BlueprintType)
class PERFORMANCECAPTUREWORKFLOW_API UPCapWorkflowCustomization : public UPCapDataAsset
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "MocapManager")
	TArray<FMocapManagerPhase> WorkflowPhases;
	
	DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnPCapCustomizationModified);
	UPROPERTY(BlueprintAssignable, Transient, Category="Performance Capture|Customization")
	FOnPCapCustomizationModified OnPCapCustomizationModified;
	
	//~ UObject interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

#endif
	//~ End UObject interface

private:
	UFUNCTION()
	static TArray<UClass*> GetAllowedPanelWidgetClasses();
	

	
};

