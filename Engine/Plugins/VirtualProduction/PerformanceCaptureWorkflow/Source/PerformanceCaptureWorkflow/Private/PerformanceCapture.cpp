// Copyright Epic Games, Inc. All Rights Reserved.

#include "PerformanceCapture.h"

#include "AssetToolsModule.h"
#include "AnalyticsEventAttribute.h"
#include "EditorUtilityWidgetBlueprint.h"
#include "IAssetTools.h"
#include "EngineAnalytics.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "Interfaces/IMainFrameModule.h"
#include "PerformanceCaptureStyle.h"
#include "PerformanceCaptureCommands.h"
#include "ISettingsModule.h"
#include "LevelEditorOutlinerSettings.h"
#include "MessageLogModule.h"
#include "PCapAssetDefinition.h"
#include "PCapDatabase.h"
#include "PCapSettings.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "ToolMenus.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

static const FName PerformanceCapturePanelTabName("PerformanceCaptureTab");

#define LOCTEXT_NAMESPACE "FPerformanceCaptureModule"

DEFINE_LOG_CATEGORY(LogPCap);

const FLazyName FPerformanceCaptureModule::MessageLogName = TEXT("PerformanceCapture");

void FPerformanceCaptureModule::StartupModule()
{
	if(ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->RegisterSettings("Project", "Plugins", "PerformanceCapture", LOCTEXT("RuntimeSettingsName", "Performance Capture"), LOCTEXT("RuntimeSettingsDescription", "Performance Capture"), GetMutableDefault<UPerformanceCaptureSettings>());
	}

	FPerformanceCaptureStyle::Initialize();
	FPerformanceCaptureStyle::ReloadTextures();

	FPerformanceCaptureCommands::Register();
	
	PluginCommands = MakeShared<FUICommandList>();

	PluginCommands->MapAction(
		FPerformanceCaptureCommands::Get().OpenPluginWindow,
		FExecuteAction::CreateRaw(this, &FPerformanceCaptureModule::PluginButtonClicked),
		FCanExecuteAction());
		
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(PerformanceCapturePanelTabName, FOnSpawnTab::CreateRaw(this, &FPerformanceCaptureModule::OnSpawnMocapManager))
		.SetDisplayName(NSLOCTEXT("PerformanceCapture", "MocapManagerTabTitle", "Mocap Manager"))
		.SetTooltipText(NSLOCTEXT("PerformanceCapture", "PerformanceCaptureTooltipText", "Open the Mocap Manager tab"))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetLevelEditorVirtualProductionCategory())
		.SetMenuType(ETabSpawnerMenuType::Enabled)
		.SetIcon(FSlateIcon(FPerformanceCaptureStyle::GetStyleSetName(), "PerformanceCapture.MocapManagerTabIcon", "PerformanceCapture.MocapManagerTabIcon.Small")
		);

	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	MessageLogModule.RegisterLogListing(MessageLogName, LOCTEXT("MessageLogPerformanceCaptureName", "PerformanceCapture"));
	
	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	AssetTools.RegisterAdvancedAssetCategory("VirtualProduction", LOCTEXT("VirtualProductionCategory", "Virtual Production"));

	IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
	MainFrameModule.OnMainFrameCreationFinished().AddRaw(this, &FPerformanceCaptureModule::OnMainFrameCreationFinished);
}

void FPerformanceCaptureModule::ShutdownModule()
{
	//Clean up settings
	if(ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->UnregisterSettings("Project", "Plugins", "PerformanceCapture");
	}

	//Clean up nomad tab spawner
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(PerformanceCapturePanelTabName);

	if (IMainFrameModule* MainFrameModule = FModuleManager::GetModulePtr<IMainFrameModule>("MainFrame"))
	{
		MainFrameModule->OnMainFrameCreationFinished().RemoveAll(this);
	}

	UToolMenus::UnRegisterStartupCallback(this);

	UToolMenus::UnregisterOwner(this);

	FPerformanceCaptureStyle::Shutdown();

	FPerformanceCaptureCommands::Unregister();

}

void FPerformanceCaptureModule::OnMainFrameCreationFinished(TSharedPtr<SWindow> InRootWindow, bool bIsNewProjectWindow)
{
	bEditorStartupInProgress = false;
}

TSharedRef<SDockTab> FPerformanceCaptureModule::OnSpawnMocapManager(const FSpawnTabArgs& SpawnTabArgs)
{
	const UPerformanceCaptureSettings* Settings = GetDefault<UPerformanceCaptureSettings>();

	if (Settings)
	{
		const bool bWidgetLoaded = Settings->MocapManagerUI.IsValid();
	

		if (FEngineAnalytics::IsAvailable())
		{
			TArray<FAnalyticsEventAttribute> Attributes;
			Attributes.Reserve(3);
			Attributes.Emplace(TEXT("Action"), TEXT("MocapManagerTabSpawned"));
			Attributes.Emplace(TEXT("Trigger"), bEditorStartupInProgress ? TEXT("LayoutRestore") : TEXT("UserInvoked"));
			Attributes.Emplace(TEXT("WidgetLoaded"), bWidgetLoaded);
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.PerformanceCapture"), Attributes);
		}

		if(bWidgetLoaded)
		{
			UEditorUtilityWidgetBlueprint* MocapManagerEW = LoadObject<UEditorUtilityWidgetBlueprint>(nullptr, *Settings->MocapManagerUI.ToString(), nullptr, LOAD_None, nullptr);
	
			TSharedRef<SDockTab> TabWidget = MocapManagerEW->SpawnEditorUITab(SpawnTabArgs);
		
			return TabWidget;
		}
	}
	
	{
	//Define message
	FText WidgetText = FText::Format(
	LOCTEXT("WindowWidgetText", "Performance Capture Project settings missing a valid UI Widget"),
	FText::FromString(TEXT("FPerformanceCaptureModule::OnSpawnMocapManager")),
	FText::FromString(TEXT("PerformanceCapture.cpp"))
	);
		//Create default tab message
		return SNew(SDockTab)
			.TabRole(ETabRole::NomadTab)
			[
				// Put your tab content here!
		
				SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(WidgetText)
				]
			];
	}
}

void FPerformanceCaptureModule::PluginButtonClicked()
{
	FGlobalTabmanager::Get()->TryInvokeTab(PerformanceCapturePanelTabName);
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FPerformanceCaptureModule, PerformanceCaptureWorkflow)

