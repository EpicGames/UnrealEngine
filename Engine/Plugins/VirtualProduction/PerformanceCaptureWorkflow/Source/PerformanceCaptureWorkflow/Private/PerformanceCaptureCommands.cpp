// Copyright Epic Games, Inc. All Rights Reserved.

#include "PerformanceCaptureCommands.h"

#define LOCTEXT_NAMESPACE "FPerformanceCaptureModule"

void FPerformanceCaptureCommands::RegisterCommands()
{
	UI_COMMAND(OpenPluginWindow, "Mocap Manager", "Open Mocap Manager", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE