// Copyright Epic Games, Inc. All Rights Reserved.

#include "PerformanceCaptureStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyle.h"

#define RootToContentDir Style->RootToContentDir

TSharedPtr<FSlateStyleSet> FPerformanceCaptureStyle::StyleInstance = nullptr;
FColor FPerformanceCaptureStyle::TypeColor(104,49,178);


void FPerformanceCaptureStyle::Initialize()
{
	if(!StyleInstance.IsValid())
	{
		StyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
	}
}

void FPerformanceCaptureStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
	ensure(StyleInstance.IsUnique());
	StyleInstance.Reset();
}

FName FPerformanceCaptureStyle::GetStyleSetName()
{
	static FName StyleSetName(TEXT("PerformanceCaptureStyle"));
	return StyleSetName;
}

const FVector2D Icon16x16(16.0f, 16.0f);
const FVector2D Icon20x20(20.0f, 20.0f);
const FVector2D Icon64x64(64.0f, 64.0f);

TSharedRef< FSlateStyleSet > FPerformanceCaptureStyle::Create()
{
	TSharedRef< FSlateStyleSet > Style = MakeShareable(new FSlateStyleSet("PerformanceCaptureStyle"));
	Style->SetContentRoot(IPluginManager::Get().FindPlugin("PerformanceCaptureWorkflow")->GetBaseDir() / TEXT("Resources"));

	//TODO: note to future self: Styles must be ClassThumbnail.Name of asset!!! */

	Style->Set("PerformanceCapture.OpenPluginWindow", new IMAGE_BRUSH_SVG(TEXT("ButtonIcon"), Icon20x20));

	Style->Set("PerformanceCapture.MocapManagerTabIcon", new IMAGE_BRUSH_SVG(TEXT("ButtonIcon"), Icon20x20));

	Style->Set("PerformanceCapture.MocapManagerTabIcon.Small", new IMAGE_BRUSH_SVG(TEXT("ButtonIcon"), Icon16x16));

	Style->Set("ClassThumbnail.PCapDataTable", new IMAGE_BRUSH_SVG(TEXT("DataTableThumbnail"), Icon64x64));

	Style->Set("ClassIcon.PCapDataTable", new IMAGE_BRUSH_SVG(TEXT("DataTableIcon"), Icon16x16));

	Style->Set("ClassThumbnail.PCapPerformerDataAsset", new IMAGE_BRUSH_SVG(TEXT("PerformerDataAsset"), Icon64x64));

	Style->Set("ClassIcon.PCapPerformerDataAsset", new IMAGE_BRUSH_SVG(TEXT("PerformerDataAsset"), Icon16x16));

	Style->Set("ClassThumbnail.PCapCharacterDataAsset", new IMAGE_BRUSH_SVG(TEXT("CharacterDataAsset"), Icon64x64));
	
	Style->Set("ClassIcon.PCapCharacterDataAsset", new IMAGE_BRUSH_SVG(TEXT("CharacterDataAsset"), Icon16x16));

	Style->Set("ClassThumbnail.PCapPropDataAsset", new IMAGE_BRUSH_SVG(TEXT("PropDataAsset"), Icon64x64));

	Style->Set("ClassIcon.PCapPropDataAsset", new IMAGE_BRUSH_SVG(TEXT("PropDataAsset"), Icon16x16));
	
	Style->Set("ClassThumbnail.PCapSessionTemplate", new IMAGE_BRUSH_SVG(TEXT("PcapSessionTemplate_64"), Icon64x64));

	Style->Set("ClassIcon.PCapSessionTemplate", new IMAGE_BRUSH_SVG(TEXT("PcapSessionTemplate_16"), Icon16x16));

	Style->Set("ClassThumbnail.PCapWorkflowCustomization", new IMAGE_BRUSH_SVG(TEXT("WorkflowIcon"), Icon64x64));

	Style->Set("ClassIcon.PCapWorkflowCustomization", new IMAGE_BRUSH_SVG(TEXT("WorkflowIcon"), Icon16x16));
	
	return Style;
}

void FPerformanceCaptureStyle::ReloadTextures()
{
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
	}
}

const ISlateStyle& FPerformanceCaptureStyle::Get()
{
	return *StyleInstance;
}
