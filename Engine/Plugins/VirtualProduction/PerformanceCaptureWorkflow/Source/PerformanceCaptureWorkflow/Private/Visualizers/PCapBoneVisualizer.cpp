// Copyright Epic Games, Inc. All Rights Reserved.


#include "PCapBoneVisualizer.h"
#include "AnimationCoreLibrary.h"
#include "SkeletalMeshAttributes.h"
#include "Animation/SkeletalMeshActor.h"
#include "Components/SkeletalMeshComponent.h"
#include "Materials/MaterialInstanceDynamic.h"


// Sets default values for this component's properties
UPCapBoneVisualiser::UPCapBoneVisualiser(const FObjectInitializer& ObjectInitializer)
{
	PrimaryComponentTick.bCanEverTick = true;
	bTickInEditor = true; //make this component tick in editor
	VisualizationType = EBoneVisualType::Joint;
	Color = FLinearColor(0.5,0.5,0.5,1.0);
	UPrimitiveComponent::SetCollisionEnabled(ECollisionEnabled::NoCollision);
	bAlwaysCreatePhysicsState = false;
	SetCastShadow(false);
}

void UPCapBoneVisualiser::OnRegister()
{
	Super::OnRegister();

	AActor* OwnerActor = GetOwner();
	
	if(OwnerActor)
	{
		SkinnedMeshComponent = OwnerActor->GetComponentByClass<USkinnedMeshComponent>();
	}

	ClearInstances();

	if(SkinnedMeshComponent)
	{
		TArray<FTransform> BoneTransforms = GetBoneTransforms(SkinnedMeshComponent);
		AddInstances(BoneTransforms, false, true, false);

		if(!DynamicMaterial)
		{
			UMaterialInterface* BaseMaterial = GetMaterial(0);
			if(BaseMaterial) //Check if the BaseMaterial is Valid. It will only be valid if a mesh has already been sent. 
			{
				DynamicMaterial = CreateDynamicMaterialInstance(0, BaseMaterial, TEXT("MaterialInstance"));
				SetMaterial(0, DynamicMaterial);
				DynamicMaterial->ClearParameterValues();
				DynamicMaterial->SetVectorParameterValue(FName(TEXT("Color")), Color);
			}
		}
	}
}

void UPCapBoneVisualiser::OnUnregister()
{
	Super::OnUnregister();

	ClearInstances();
}

void UPCapBoneVisualiser::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
	
	if(SkinnedMeshComponent)
	{
		switch (VisualizationType)
		{
		case EBoneVisualType::Bone:
			{
				BatchUpdateInstancesTransforms(0, GetBoneTransforms(SkinnedMeshComponent), true, true, false);
			}
			break;
		case EBoneVisualType::Joint:
			{
				BatchUpdateInstancesTransforms(0, GetJointTransforms(SkinnedMeshComponent), true, true, false);
			}

			break;
		default:
			checkNoEntry();
		}
	}
}

TArray<FTransform> UPCapBoneVisualiser::GetJointTransforms(USkinnedMeshComponent* InSkinnedMeshComponent) const
{
	TArray<FTransform> FoundTransforms;
	
	if(InSkinnedMeshComponent)
	{
		FTransform BoneTransform;

		for (int32 i = 0; i < InSkinnedMeshComponent->GetNumBones(); i++ )
		{
			BoneTransform = InSkinnedMeshComponent->GetBoneTransform(InSkinnedMeshComponent->GetBoneName(i));
			FoundTransforms.Add(BoneTransform);
		}
	}
	return FoundTransforms;
}

TArray<FTransform> UPCapBoneVisualiser::GetBoneTransforms(USkinnedMeshComponent* InSkinnedMeshComponent) const
{
	TArray<FTransform> FoundTransforms;
	if(InSkinnedMeshComponent)
	{
		FTransform BoneTransform;
		FTransform ParentBoneTransform;
		double BoneLength;
		FVector Aim = FVector( 0, 0, 1);

		for (int32 i = 0; i < InSkinnedMeshComponent->GetNumBones(); i++ )
		{
			BoneTransform = InSkinnedMeshComponent->GetBoneTransform(InSkinnedMeshComponent->GetBoneName(i));
			ParentBoneTransform = InSkinnedMeshComponent->GetBoneTransform(InSkinnedMeshComponent->GetParentBone(InSkinnedMeshComponent->GetBoneName(i)));
			BoneLength = FVector::Distance(BoneTransform.GetLocation(), ParentBoneTransform.GetLocation());
			BoneTransform.SetScale3D(FVector(1, 1, BoneLength));
			FQuat DiffRotation = AnimationCore::SolveAim(BoneTransform, ParentBoneTransform.GetLocation(), Aim.GetSafeNormal(), false, FVector(1, 1, 1), float(0));
			BoneTransform.SetRotation(DiffRotation);
			FoundTransforms.Add(BoneTransform);
		}
	}
	return FoundTransforms;
}

void UPCapBoneVisualiser::UpdateColor(FLinearColor NewColor)
{
	Color = NewColor;
	if(DynamicMaterial)
	{
		DynamicMaterial->SetVectorParameterValue(FName(TEXT("Color")), Color);
	}
}

void UPCapBoneVisualiser::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if(PropertyChangedEvent.Property != nullptr)
	{
		if(PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPCapBoneVisualiser, Color))
		{
			UpdateColor(Color);
		}
	}
}


