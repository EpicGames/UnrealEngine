// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "PCapBoneVisualizer.generated.h"

class USkinnedMeshComponent;
class UMaterialInstanceDynamic;

UENUM(BlueprintType)
enum class EBoneVisualType : uint8
{
	Joint	UMETA(DisplayName = "Joint"),
	Bone	UMETA(DisplayName = "Bone"),
};

/** Instanced Static Mesh Component for drawing bones and joints on a Skinned Mesh component. Only usable with Actors that contain at least one Skinned Mesh component*/
UCLASS(ClassGroup=(PerformanceCapture), meta=(BlueprintSpawnableComponent), Transient, DisplayName = "PerformanceCaptureVisualizer")
class PERFORMANCECAPTUREWORKFLOW_API UPCapBoneVisualiser : public UInstancedStaticMeshComponent
{
	GENERATED_BODY()

public:
	// Sets default values for this component's properties
	UPCapBoneVisualiser(const FObjectInitializer& ObjectInitializer);
	
	// Called every frame
	virtual void TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;

	/** Get all the transforms from the Skinned Mesh component. */
	TArray<FTransform> GetJointTransforms(USkinnedMeshComponent* InSkinnedMeshComponent) const;

	/** Get all the bones' transforms for drawing a joint (point to point mesh). */
	TArray<FTransform> GetBoneTransforms(USkinnedMeshComponent* InSkinnedMeshComponent) const;
	
	/** Whether to draw joints along the length of the bone, or just at the pivot or scaled by the distance to the parent bone. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="PerformanceCapture|Visualization")
	EBoneVisualType VisualizationType;

	/** Color to use on the instanced static meshes. Can only be set during construction. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter="UpdateColor", Category="PerformanceCapture|Visualization")
	FLinearColor Color;

	/** Dynamic Material to use on the instanced static meshes. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="PerformanceCapture|Visualization")
	TObjectPtr<UMaterialInstanceDynamic> DynamicMaterial;

	/**
	 * Update the color for this bone visaulizer.
	 * @param NewColor New color to apply in the material.
	 */
	UFUNCTION(BlueprintCallable, Category="PerformanceCapture|Visualization")
	void UpdateColor(FLinearColor NewColor);

#if WITH_EDITOR

	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	
#endif

protected:
	virtual void OnRegister() override;
	virtual void OnUnregister() override;

private:
	UPROPERTY()
	TObjectPtr<USkinnedMeshComponent> SkinnedMeshComponent;
};
