// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "View/MVVMViewModelContextResolver.h"
#include "PCapSubsystem.h"
#include "PCapMVVMResolver.generated.h"

/**
 * Editor-only resolver class
 */
UCLASS()
class UPCapMVVMResolver : public UMVVMViewModelContextResolver
{
	GENERATED_BODY()

#if WITH_EDITOR
	
	virtual UObject* CreateInstance(const UClass* ExpectedType, const UUserWidget* UserWidget, const UMVVMView* View) const override;

#endif
};
