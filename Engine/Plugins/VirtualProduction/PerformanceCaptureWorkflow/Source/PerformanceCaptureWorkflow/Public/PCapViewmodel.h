// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MVVMViewModelBase.h"
#include "PCapViewmodel.generated.h"

/**
 * 
 */
UCLASS(Blueprintable, Abstract, DisplayName="PCap Viewmodel",meta = (ShowWorldContextPin))
class PERFORMANCECAPTUREWORKFLOW_API UPCapViewmodel : public UMVVMViewModelBase
{
	GENERATED_BODY()
public:

	/** Initialise the viewmodel. Implemented in Blueprint. */
	UFUNCTION(BlueprintCallable, BlueprintImplementableEvent, Category="Performance Capture|Viewmodel")
	void Initialize();
};
