// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPlacementModeModule.h"
#include "Modules/ModuleManager.h"

#define UE_API PERFORMANCECAPTUREWORKFLOW_API

DECLARE_LOG_CATEGORY_EXTERN(LogPCap, Log, All);
class FPlacementModeID;

class FPerformanceCaptureModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	
	/** This function will be bound to Command (by default it will bring up plugin window) */
	void PluginButtonClicked();

	/** Message Log Name for the PerformanceCapture module. */
	UE_API static const FLazyName MessageLogName; // "PerformanceCapture"

private:
	TSharedRef<class SDockTab> OnSpawnMocapManager(const class FSpawnTabArgs& SpawnTabArgs);

	void OnMainFrameCreationFinished(TSharedPtr<class SWindow> InRootWindow, bool bIsNewProjectWindow);

	TSharedPtr<class FUICommandList> PluginCommands;

	bool bEditorStartupInProgress = true;
};

#undef UE_API
