// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "PerformanceCaptureStyle.h"

class FPerformanceCaptureCommands : public TCommands<FPerformanceCaptureCommands>
{
public:

	FPerformanceCaptureCommands()
		: TCommands<FPerformanceCaptureCommands>(TEXT("PerformanceCapture"), NSLOCTEXT("Contexts", "PerformanceCapture", "PerformanceCaptureWorkflow Plugin"), NAME_None, FPerformanceCaptureStyle::GetStyleSetName())
	{
	}

	// TCommands<> interface
	virtual void RegisterCommands() override;

public:
	TSharedPtr< FUICommandInfo > OpenPluginWindow;
};