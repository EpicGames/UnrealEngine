// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateStyle.h"

class ISlateStyle;
struct FSlateBrush;

/**  */
class FPerformanceCaptureStyle
{
public:

	static void Initialize();

	static void Shutdown();

	/** reloads textures used by slate renderer */
	static void ReloadTextures();

	/** @return The Slate style set for the editor */
	static const ISlateStyle& Get();

	static FName GetStyleSetName();

	static FColor TypeColor;


private:

	static TSharedRef< class FSlateStyleSet > Create();
	
	static TSharedPtr< class FSlateStyleSet > StyleInstance;
};
