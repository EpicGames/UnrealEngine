// Copyright Epic Games, Inc. All Rights Reserved.
using UnrealBuildTool;

public class PerformanceCaptureWorkflowRuntime : ModuleRules
{
    public PerformanceCaptureWorkflowRuntime(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
        ShortName = "PCWFR";

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core"
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "CoreUObject",
                "Engine",
                "Slate",
                "SlateCore",
                "LiveLink",
                "LiveLinkInterface",
                "LiveLinkAnimationCore",
				"PerformanceCaptureCore", 
				"IKRig"
            }
        );
    }
}