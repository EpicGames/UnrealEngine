// Copyright Epic Games, Inc. All Rights Reserved.


#include "AnimNode_LiveLinkProp.h"

#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimTrace.h"
#include "Features/IModularFeatures.h"
#include "ILiveLinkClient.h"
#include "Roles/LiveLinkAnimationRole.h"
#include "Roles/LiveLinkAnimationTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_LiveLinkProp)

FAnimNode_LiveLinkProp::FAnimNode_LiveLinkProp()
	: LiveLinkClient_AnyThread(nullptr)
	, CachedDeltaTime(0.0f)
{
}

void FAnimNode_LiveLinkProp::BuildPoseFromAnimData(const FLiveLinkSubjectFrameData& LiveLinkData, FPoseContext& Output)
{
	const FLiveLinkSkeletonStaticData* SkeletonData = LiveLinkData.StaticData.Cast<FLiveLinkSkeletonStaticData>();
	const FLiveLinkAnimationFrameData* FrameData = LiveLinkData.FrameData.Cast<FLiveLinkAnimationFrameData>();
	check(SkeletonData);
	check(FrameData);

	//Get bones names
	const TArray<FName>& SourceBoneNames = SkeletonData->GetBoneNames();

	if (SkeletonData->BoneNames.Num() == FrameData->Transforms.Num())
	{
		//Iterate through bones in LiveLink Data
		for (int32 i = 0; i < SourceBoneNames.Num(); ++i)
		{
			FName BoneName = SourceBoneNames[i];
		
			FTransform LiveLinkTransform = FrameData->Transforms[i];
			LiveLinkTransform.SetLocation(FrameData->Transforms[i].GetLocation() * GlobalScale);

			FTransform BoneTransform = OffsetTransform * LiveLinkTransform ;

			BoneTransform.SetLocation(BoneTransform.GetLocation() + DynamicConstraintOffset);

			int32 MeshIndex = Output.Pose.GetBoneContainer().GetPoseBoneIndexForBoneName(BoneName);
			if (MeshIndex != INDEX_NONE)
			{
				FCompactPoseBoneIndex CompactPoseBoneIndex = Output.Pose.GetBoneContainer().MakeCompactPoseIndex(FMeshPoseBoneIndex(MeshIndex));
				if(CompactPoseBoneIndex != INDEX_NONE)
				{
					Output.Pose[CompactPoseBoneIndex] = BoneTransform;
				}
			}
		}
	}
	
	CachedDeltaTime = 0.0f;
}

void FAnimNode_LiveLinkProp::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	InputPose.Initialize(Context);

	if (CachedLiveLinkFrameData.IsValid() == false)
	{
		CachedLiveLinkFrameData  = MakeShared<FLiveLinkSubjectFrameData>();
	}
}

void FAnimNode_LiveLinkProp::PreUpdate(const UAnimInstance* InAnimInstance)
{
	ILiveLinkClient* ThisFrameClient = nullptr;
	IModularFeatures& ModularFeatures  = IModularFeatures::Get();
	if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		ThisFrameClient = &IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
	}
	LiveLinkClient_AnyThread = ThisFrameClient;
}

void FAnimNode_LiveLinkProp::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	InputPose.Update(Context);

	GetEvaluateGraphExposedInputs().Execute(Context);

	CachedDeltaTime += Context.GetDeltaTime();

	TRACE_ANIM_NODE_VALUE(Context, TEXT("SubjectName"), LiveLinkSubjectName.Name);
}

void FAnimNode_LiveLinkProp::Evaluate_AnyThread(FPoseContext& Output)
{
	InputPose.Evaluate(Output);

	if (!LiveLinkClient_AnyThread)
	{
		return;
	}

	FLiveLinkSubjectFrameData SubjectFrameData;

	if(bDoLiveLinkEvaluation)
	{
		// Invalidate cached evaluated Role to make sure we have a valid one during the last evaluation when using it
		CachedEvaluatedRole = nullptr;

		TSubclassOf<ULiveLinkRole> SubjectRole = LiveLinkClient_AnyThread->GetSubjectRole_AnyThread(LiveLinkSubjectName);
		if (SubjectRole)
		{
			if(LiveLinkClient_AnyThread->DoesSubjectSupportsRole_AnyThread(LiveLinkSubjectName, ULiveLinkAnimationRole::StaticClass()))
			{
				//Process the animation data
				if(LiveLinkClient_AnyThread->EvaluateFrame_AnyThread(LiveLinkSubjectName, ULiveLinkAnimationRole::StaticClass(), SubjectFrameData))
				{
					BuildPoseFromAnimData(SubjectFrameData, Output);
					CachedEvaluatedRole = ULiveLinkAnimationRole::StaticClass();
				}
			}
			CachedLiveLinkFrameData->StaticData = MoveTemp(SubjectFrameData.StaticData);
			CachedLiveLinkFrameData->FrameData = MoveTemp(SubjectFrameData.FrameData);
		}
	}
	else
	{
		if(CachedLiveLinkFrameData && CachedEvaluatedRole)
		{
			if (CachedEvaluatedRole == ULiveLinkAnimationRole::StaticClass())
			{
				BuildPoseFromAnimData(*CachedLiveLinkFrameData, Output);
			}
		}
	}
}

void FAnimNode_LiveLinkProp::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	Super::CacheBones_AnyThread(Context);
	InputPose.CacheBones(Context);
}

void FAnimNode_LiveLinkProp::GatherDebugData(FNodeDebugData& DebugData)
{
	FString DebugLine = FString::Printf(TEXT("LiveLink Prop - SubjectName: %s"), *LiveLinkSubjectName.ToString());

	DebugData.AddDebugItem(DebugLine);
	InputPose.GatherDebugData(DebugData);
}

