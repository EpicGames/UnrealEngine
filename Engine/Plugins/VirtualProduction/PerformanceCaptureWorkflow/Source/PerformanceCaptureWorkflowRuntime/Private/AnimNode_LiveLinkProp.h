// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimNodeBase.h"

#include "CoreMinimal.h"
#include "LiveLinkTypes.h"
#include "Templates/SubclassOf.h"

#include "AnimNode_LiveLinkProp.generated.h"

class ILiveLinkClient;
class ULiveLinkRole;
struct FLiveLinkSubjectFrameData;

/**
 * This animnode is exclusively for Mocap props - single bone skeleton meshes. Not exposed to the animation graph.
 */

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_LiveLinkProp : public FAnimNode_Base
{
	GENERATED_BODY()

public:
	/** Input pose */
	UPROPERTY(EditAnywhere, Category = "Performance Capture|Prop")
	FPoseLink InputPose;

	/** The Live Link subject to use. */
	UPROPERTY(EditAnywhere, Category = "Performance Capture|Prop")
	FLiveLinkSubjectName LiveLinkSubjectName;

	/** Bool to control evaluation of animation. */
	UPROPERTY(EditAnywhere, Transient, Category = "Performance Capture|Prop")
	bool bDoLiveLinkEvaluation = true;

	/* Transform to apply local space to offset to the incoming Live Link data. */
	UPROPERTY(EditAnywhere, Category = "Performance Capture|Prop")
	FTransform OffsetTransform = FTransform::Identity;
	
	/* Global scale to the location*/
	UPROPERTY(EditAnywhere, Category = "Performance Capture|Prop")
	float GlobalScale = 1.0f;

	/* Location to apply in parent bone space to offset the incoming Live Link data. */
	UPROPERTY(EditAnywhere, Category = "Performance Capture|Prop")
	FVector DynamicConstraintOffset = FVector::ZeroVector;
	
	FAnimNode_LiveLinkProp();

	//~ FAnimNode_Base interface
	virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	virtual void PreUpdate(const UAnimInstance* InAnimInstance) override;
	virtual void Update_AnyThread(const FAnimationUpdateContext & Context) override;
	virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext & Context) override;
	virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	virtual bool HasPreUpdate() const override { return true; };
	//~ End of FAnimNode_Base interface

protected:
	
	void BuildPoseFromAnimData(const FLiveLinkSubjectFrameData& LiveLinkData, FPoseContext& Output);

private:

	ILiveLinkClient* LiveLinkClient_AnyThread;

	float CachedDeltaTime;

	TSharedPtr<FLiveLinkSubjectFrameData> CachedLiveLinkFrameData;

	TSubclassOf<class ULiveLinkRole> CachedEvaluatedRole;
};
