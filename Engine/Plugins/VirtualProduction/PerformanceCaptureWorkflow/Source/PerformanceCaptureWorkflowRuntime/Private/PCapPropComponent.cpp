// Copyright Epic Games, Inc. All Rights Reserved.


#include "PCapPropComponent.h"

#include "CaptureCharacter.h"
#include "Components/SkeletalMeshComponent.h"
#include "ILiveLinkClient.h"
#include "PCapPropLiveLinkAnimInstance.h"
#include "PerformanceCaptureWorkflowRuntime.h"
#include "RetargetComponent.h"
#include "GameFramework/Actor.h"
#include "Components/SceneComponent.h"
#include "Features/IModularFeatures.h"
#include "Retargeter/IKRetargeter.h"
#include "Roles/LiveLinkAnimationRole.h"
#include "Roles/LiveLinkTransformRole.h"
#include "Roles/LiveLinkTransformTypes.h"


// Sets default values for this component's properties
UPCapPropComponent::UPCapPropComponent()
	: bIsDirty(true)
{
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bStartWithTickEnabled = true;
	PrimaryComponentTick.TickGroup = ETickingGroup::TG_PrePhysics;
	bTickInEditor = true;
}

UPCapPropComponent::~UPCapPropComponent()
{
	//TODO: Might have to do something when exiting PIE here
}

USceneComponent* UPCapPropComponent::GetControlledComponent() const
{
	if(ControlledComponent.GetComponent(GetOwner()))
	{
		return Cast<USceneComponent>(ControlledComponent.GetComponent(GetOwner()));
	}

	return nullptr;
}

void UPCapPropComponent::SetControlledComponent(USceneComponent* InComponent)
{
	//Check the offered new component is inside the same actor as this prop component
	if(InComponent->GetOwner() == this->GetOwner())
	{
		ControlledComponent.OverrideComponent = InComponent;
	}
}

void UPCapPropComponent::SetLiveLinkSubject(FLiveLinkSubjectName Subject)
{
	SubjectName = Subject;

	//If there is a Skeletal Mesh controlled by an Anim Instance, reinit.
	InitiateAnimation();
}

FLiveLinkSubjectName UPCapPropComponent::GetLiveLinkSubject() const
{
	return SubjectName;
}

void UPCapPropComponent::SetEvaluateLiveLinkData(bool bEvaluate)
{
	bEvaluateLiveLink = bEvaluate;

	//If there is a Skeletal Mesh controlled by an Anim Instance, reinit.
	USkeletalMeshComponent* ControlledMesh = Cast<USkeletalMeshComponent>(ControlledComponent.GetComponent(GetOwner()));
	if(IsValid(ControlledMesh))
	{
		UPCapPropLiveLinkAnimInstance* LiveLinkAnimInstance = Cast<UPCapPropLiveLinkAnimInstance>(
			ControlledMesh->GetAnimInstance());
		if(LiveLinkAnimInstance)
		{
			LiveLinkAnimInstance->EnableLiveLinkEvaluation(bEvaluateLiveLink);
		}
	}
}

bool UPCapPropComponent::GetEvaluateLiveLinkData()
{
	return bEvaluateLiveLink;
}

void UPCapPropComponent::SetOffsetTransform(FTransform NewOffset)
{
	OffsetTransform = NewOffset;
	
	//If there is a Skeletal Mesh controlled by an Anim Instance, reinit.
	TObjectPtr<USkeletalMeshComponent> ControlledMesh = Cast<USkeletalMeshComponent>(ControlledComponent.GetComponent(GetOwner()));
	if(IsValid(ControlledMesh))
	{
		if(UPCapPropLiveLinkAnimInstance* LiveLinkAnimInstance = Cast<UPCapPropLiveLinkAnimInstance>(ControlledMesh->GetAnimInstance()))
		{
			LiveLinkAnimInstance->SetOffsetTransform(OffsetTransform);
		}
	}
}

void UPCapPropComponent::SetGlobalScale(float NewGlobalScale)
{
	GlobalScale = NewGlobalScale;
	
	//If there is a Skeletal Mesh controlled by an Anim Instance pass the new global scale
	TObjectPtr<USkeletalMeshComponent> ControlledMesh = Cast<USkeletalMeshComponent>(ControlledComponent.GetComponent(GetOwner()));
	if(IsValid(ControlledMesh))
	{
		if(UPCapPropLiveLinkAnimInstance* LiveLinkAnimInstance = Cast<UPCapPropLiveLinkAnimInstance>(ControlledMesh->GetAnimInstance()))
		{
			LiveLinkAnimInstance->SetGlobalScale(GlobalScale);
		}
	}
}

TArray<FName> UPCapPropComponent::GetAttachBones() const
{
	TArray<FName> FoundIKChainBones;
	
	for (TSoftObjectPtr<ACaptureCharacter> Character : DynamicAttachmentCharacters)
	{
		if (Character.IsValid())
		{
			if (const UIKRetargeter* Retargeter = Character.Get()->GetRetargetComponent()->RetargetAsset)
			{
				if (const UIKRigDefinition* TargetIKRig = Retargeter->GetIKRig(ERetargetSourceOrTarget::Target))
				{
					TArray<FName> ChainNames = TargetIKRig->GetRetargetChainNames();

					for (FName ChainName : ChainNames)
					{
						if (bAllowOnlyIKBones)
						{
							if (const FBoneChain* Chain = TargetIKRig->GetRetargetChainByName(ChainName); Chain->IKGoalName != NAME_None)
							{
								FoundIKChainBones.AddUnique(Chain->StartBone.BoneName);
								FoundIKChainBones.AddUnique(Chain->EndBone.BoneName);
							}
						}
						else
						{
							const FBoneChain* Chain = TargetIKRig->GetRetargetChainByName(ChainName);
							FoundIKChainBones.AddUnique(Chain->StartBone.BoneName);
							FoundIKChainBones.AddUnique(Chain->EndBone.BoneName);
						}
					}
				}
			}
				
		}
	}
	return FoundIKChainBones;
}

FVector UPCapPropComponent::CalculateDynamicOffset_Implementation()
{
	FVector ReturnValue = FVector::ZeroVector;
	return ReturnValue;
}

void UPCapPropComponent::OnRegister()
{
	Super::OnRegister();
	bIsDirty = true;

	GetOwner()->Tags.AddUnique(TEXT("PCapProp"));

	//Enforce mobility on the controlled component
	if(ControlledComponent.GetComponent(GetOwner()))
	{
		Cast<USceneComponent>(ControlledComponent.GetComponent(GetOwner()))->SetMobility(EComponentMobility::Movable);
	}

	TObjectPtr<USkeletalMeshComponent> ControlledMesh = Cast<USkeletalMeshComponent>(ControlledComponent.GetComponent(GetOwner()));
	if(ControlledMesh && bIsDirty)
	{
		InitiateAnimation();
	}
}

void UPCapPropComponent::DestroyComponent(bool bPromoteChildren)
{
	ResetAnimInstance();
	
	Super::DestroyComponent(bPromoteChildren);
}

#if WITH_EDITOR
void UPCapPropComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if(PropertyChangedEvent.Property != nullptr)
	{
		if(ControlledComponent.GetComponent(GetOwner()))
		{
			Cast<USceneComponent>(ControlledComponent.GetComponent(GetOwner()))->SetMobility(EComponentMobility::Movable);
		}
		if(PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPCapPropComponent, OffsetTransform))
		{
			SetOffsetTransform(OffsetTransform);
		}
		if(PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPCapPropComponent, ControlledComponent))
		{
			ResetAnimInstance();
			bIsDirty = true;
		}
		if(PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPCapPropComponent, SubjectName))
		{
			ResetAnimInstance();
			bIsDirty = true;
		}
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

void UPCapPropComponent::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	// Calculate the dynamic constraint. Will call to the BP implmented method if there is one available. 
	DynamicConstraintOffset = CalculateDynamicOffset();
	
	//Check if spawnable
	if(!bIsSpawnableCache.IsSet() || bIsDirty)
	{
		static const FName SequencerActorTag(TEXT("SequencerActor"));
		const AActor* OwningActor = GetOwner();

		bIsSpawnableCache = OwningActor && OwningActor->ActorHasTag(SequencerActorTag);

		if(*bIsSpawnableCache)
		{
			bEvaluateLiveLink = false;
			bIsControlledBySequencer = true;
		}
	}
	
	FTransform GlobalScaleTransform = FTransform::Identity;
	GlobalScaleTransform.SetScale3D(FVector(GlobalScale, GlobalScale, GlobalScale));

	if(!bIsControlledBySequencer)
	{
		// If we are controlling a static mesh, follow this path of applying a relative transform to the controlled mesh from the LiveLink data
		const TObjectPtr<UStaticMeshComponent> ControlledMesh = Cast<UStaticMeshComponent>(ControlledComponent.GetComponent(GetOwner()));
		if(ControlledMesh)
		{
			FLiveLinkSubjectFrameData SubjectFrameData;

			ILiveLinkClient& LiveLinkClient = IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
	
			LiveLinkClient.EvaluateFrame_AnyThread(SubjectName, LiveLinkClient.GetSubjectRole_AnyThread(SubjectName), SubjectFrameData);
	
			if(LiveLinkClient.GetSubjectRole_AnyThread(SubjectName) == ULiveLinkAnimationRole::StaticClass())
			{
				const FLiveLinkAnimationFrameData* FrameData = SubjectFrameData.FrameData.Cast<FLiveLinkAnimationFrameData>();
				if(FrameData && bEvaluateLiveLink)
				{
					CachedLiveLinkTransform = FrameData->Transforms[0];
					FinalTransform = CachedLiveLinkTransform;
				}

				if (bUseDynamicConstraint)
				{
					FinalTransform.SetComponents(CachedLiveLinkTransform.GetRotation(),(CachedLiveLinkTransform.GetLocation() * GlobalScale) + DynamicConstraintOffset, CachedLiveLinkTransform.GetScale3D());
				}
				else
				{
					FinalTransform.SetComponents(CachedLiveLinkTransform.GetRotation(),(CachedLiveLinkTransform.GetLocation() * GlobalScale), CachedLiveLinkTransform.GetScale3D());
				}
				
				GetControlledComponent()->SetRelativeTransform(OffsetTransform * FinalTransform, false, nullptr, ETeleportType::None);
			}
			else if(LiveLinkClient.GetSubjectRole_AnyThread(SubjectName) == ULiveLinkTransformRole::StaticClass())
			{
				const FLiveLinkTransformFrameData* FrameData = SubjectFrameData.FrameData.Cast<FLiveLinkTransformFrameData>();
				if(FrameData && bEvaluateLiveLink)
				{
					CachedLiveLinkTransform = FrameData->Transform;
				}
				
				if (bUseDynamicConstraint)
				{
					FinalTransform.SetComponents(CachedLiveLinkTransform.GetRotation(),(CachedLiveLinkTransform.GetLocation() * GlobalScale) + DynamicConstraintOffset, CachedLiveLinkTransform.GetScale3D());
				}
				else
				{
					FinalTransform.SetComponents(CachedLiveLinkTransform.GetRotation(),(CachedLiveLinkTransform.GetLocation() * GlobalScale), CachedLiveLinkTransform.GetScale3D());
				}
				
				GetControlledComponent()->SetRelativeTransform(OffsetTransform * FinalTransform, false, nullptr, ETeleportType::None);
			}
		}
		//If we are controlling a Skelmesh and receiving a transform role, apply this route. 
		const TObjectPtr<USkeletalMeshComponent> ControlledSkelmesh = Cast<USkeletalMeshComponent>(ControlledComponent.GetComponent(GetOwner()));
		if(ControlledSkelmesh)
		{
			FLiveLinkSubjectFrameData SkelSubjectFrameData;

			ILiveLinkClient& SkelLiveLinkClient = IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);

			SkelLiveLinkClient.EvaluateFrame_AnyThread(SubjectName, SkelLiveLinkClient.GetSubjectRole_AnyThread(SubjectName), SkelSubjectFrameData);
			if(SkelLiveLinkClient.GetSubjectRole_AnyThread(SubjectName) == ULiveLinkTransformRole::StaticClass())
			{
				const FLiveLinkTransformFrameData* FrameData = SkelSubjectFrameData.FrameData.Cast<FLiveLinkTransformFrameData>();
				if(FrameData && bEvaluateLiveLink)
				{
					CachedLiveLinkTransform = FrameData->Transform;
				}

				if (bUseDynamicConstraint)
				{
					FinalTransform.SetComponents(CachedLiveLinkTransform.GetRotation(),(CachedLiveLinkTransform.GetLocation() * GlobalScale) + DynamicConstraintOffset, CachedLiveLinkTransform.GetScale3D());
				}
				else
				{
					FinalTransform.SetComponents(CachedLiveLinkTransform.GetRotation(),(CachedLiveLinkTransform.GetLocation() * GlobalScale), CachedLiveLinkTransform.GetScale3D());
				}
				
				GetControlledComponent()->SetRelativeTransform(OffsetTransform * FinalTransform, false, nullptr, ETeleportType::None);
			}
			
			if((SkelLiveLinkClient.GetSubjectRole_AnyThread(SubjectName) == ULiveLinkAnimationRole::StaticClass()) && bIsDirty)
			{
				InitiateAnimation();
			}
			
			if (bUseDynamicConstraint && ControlledSkelmesh)
			{
				if(UPCapPropLiveLinkAnimInstance* AnimInstance = Cast<UPCapPropLiveLinkAnimInstance>(ControlledSkelmesh->GetAnimInstance()))
				{
					AnimInstance->SetDynamicConstraintVector(DynamicConstraintOffset); // This is only ever applied in the space of the root bone of the skeleton.
				}
			}
		}
	}
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
}

void UPCapPropComponent::InitiateAnimation()
{
	TObjectPtr<USkeletalMeshComponent> ControlledMesh = Cast<USkeletalMeshComponent>(ControlledComponent.GetComponent(GetOwner()));
	ILiveLinkClient& LiveLinkClient = IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
	
	FTransform GlobalScaleTransform = FTransform::Identity;
	GlobalScaleTransform.SetScale3D(FVector(GlobalScale, GlobalScale, GlobalScale));
	
	//If we are controlling a Skeletal Mesh and the Live Link data is of the animation role, initialize animation instance
	if(IsValid(ControlledMesh) && LiveLinkClient.GetSubjectRole_AnyThread(SubjectName) == ULiveLinkAnimationRole::StaticClass())
	{
		ControlledMesh->SetAnimInstanceClass(UPCapPropLiveLinkAnimInstance::StaticClass());
		ControlledMesh->InitAnim(true /*bForceReinit*/);
		ControlledMesh->SetUpdateAnimationInEditor(true);
		ControlledMesh->bPropagateCurvesToFollowers = true;
		ControlledMesh->VisibilityBasedAnimTickOption = EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones;
		if(ControlledMesh)
		{
			UPCapPropLiveLinkAnimInstance* AnimInstance = Cast<UPCapPropLiveLinkAnimInstance>(ControlledMesh->GetAnimInstance());
			if(AnimInstance)
			{
				AnimInstance->SetSubject(SubjectName);
				AnimInstance->EnableLiveLinkEvaluation(bEvaluateLiveLink);
				AnimInstance->SetOffsetTransform(OffsetTransform);
				AnimInstance->SetGlobalScale(GlobalScale);
				AnimInstance->SetDynamicConstraintVector(FVector::ZeroVector);
				bIsDirty=false;
			}
		}
	}
}

void UPCapPropComponent::ResetAnimInstance() const
{
	TObjectPtr<USkeletalMeshComponent> ControlledMesh = Cast<USkeletalMeshComponent>(ControlledComponent.GetComponent(GetOwner()));
	if(ControlledMesh)
	{
		ControlledMesh->SetAnimInstanceClass(nullptr);
		ControlledMesh->InitAnim(true /*bForceReinit*/);
		ControlledMesh->SetUpdateAnimationInEditor(true);
	}
}
