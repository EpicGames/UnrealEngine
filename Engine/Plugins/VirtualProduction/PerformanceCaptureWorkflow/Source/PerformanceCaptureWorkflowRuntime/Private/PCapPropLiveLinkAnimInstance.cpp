// Copyright Epic Games, Inc. All Rights Reserved.


#include "PCapPropLiveLinkAnimInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCapPropLiveLinkAnimInstance)

/*------------------------------------------------------------------------------
	Anim Instance Proxy.
------------------------------------------------------------------------------*/


void FPCapPropLiveLinkAnimInstanceProxy::Initialize(UAnimInstance* InAnimInstance)
{
	FAnimInstanceProxy::Initialize(InAnimInstance);
	FAnimationInitializeContext InitContext(this);
	PoseNode.Initialize_AnyThread(InitContext);
}

void FPCapPropLiveLinkAnimInstanceProxy::PreUpdate(UAnimInstance* InAnimInstance, float DeltaSeconds)
{
	Super::PreUpdate(InAnimInstance, DeltaSeconds);

	if(PoseNode.HasPreUpdate())
	{
		PoseNode.PreUpdate(InAnimInstance);
	}
}

bool FPCapPropLiveLinkAnimInstanceProxy::Evaluate(FPoseContext& Output)
{
	PoseNode.Evaluate_AnyThread(Output);
	
	return true;
}

void FPCapPropLiveLinkAnimInstanceProxy::UpdateAnimationNode(const FAnimationUpdateContext& InContext)
{
	UpdateCounter.Increment();

	PoseNode.Update_AnyThread(InContext);
}

/*------------------------------------------------------------------------------
	Anim Instance
------------------------------------------------------------------------------*/
FAnimInstanceProxy* UPCapPropLiveLinkAnimInstance::CreateAnimInstanceProxy()
{
	return new FPCapPropLiveLinkAnimInstanceProxy(this);
}

void UPCapPropLiveLinkAnimInstance::DestroyAnimInstanceProxy(FAnimInstanceProxy* InProxy)
{
	Super::DestroyAnimInstanceProxy(InProxy);
}
