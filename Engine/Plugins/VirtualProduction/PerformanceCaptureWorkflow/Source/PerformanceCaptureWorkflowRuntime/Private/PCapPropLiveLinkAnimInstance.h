// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNode_LiveLinkProp.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimInstance.h"
#include "PCapPropLiveLinkAnimInstance.generated.h"


USTRUCT()
struct FPCapPropLiveLinkAnimInstanceProxy : public FAnimInstanceProxy
{
public:
	friend struct FAnimNode_LiveLinkProp;

	GENERATED_BODY()

	FPCapPropLiveLinkAnimInstanceProxy()
	{
	}

	FPCapPropLiveLinkAnimInstanceProxy(UAnimInstance* InAnimInstance)
		: FAnimInstanceProxy(InAnimInstance)
	{
	}

	virtual void Initialize(UAnimInstance* InAnimInstance) override;
	virtual void PreUpdate(UAnimInstance* InAnimInstance, float DeltaSeconds) override;
	virtual bool Evaluate(FPoseContext& Output) override;
	virtual void UpdateAnimationNode(const FAnimationUpdateContext& InContext) override;

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_LiveLinkProp PoseNode;
};
/**
 * 
 */
UCLASS(transient, NotBlueprintable)
class PERFORMANCECAPTUREWORKFLOWRUNTIME_API UPCapPropLiveLinkAnimInstance : public UAnimInstance
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintCallable, Category = "Performance Capture|Prop")
	void SetSubject(FLiveLinkSubjectName SubjectName)
	{
		GetProxyOnGameThread<FPCapPropLiveLinkAnimInstanceProxy>().PoseNode.LiveLinkSubjectName = SubjectName;
	}

	UFUNCTION(BlueprintCallable, Category = "Performance Capture|Prop")
	void EnableLiveLinkEvaluation(bool bDoEnable)
	{
		GetProxyOnGameThread<FPCapPropLiveLinkAnimInstanceProxy>().PoseNode.bDoLiveLinkEvaluation = bDoEnable;
	}
	
	UFUNCTION(BlueprintCallable, Category = "Performance Capture|Prop")
	void SetOffsetTransform(FTransform Offset)
	{
		GetProxyOnGameThread<FPCapPropLiveLinkAnimInstanceProxy>().PoseNode.OffsetTransform = Offset;
	}

	UFUNCTION(BlueprintCallable, Category = "Performance Capture|Prop")
	void SetDynamicConstraintVector(FVector DynamicOffset)
	{
		GetProxyOnGameThread<FPCapPropLiveLinkAnimInstanceProxy>().PoseNode.DynamicConstraintOffset = DynamicOffset;
	}
	
	UFUNCTION(BlueprintCallable, Category = "Performance Capture|Prop")
	void SetGlobalScale(float GlobalScale)
	{
		GetProxyOnGameThread<FPCapPropLiveLinkAnimInstanceProxy>().PoseNode.GlobalScale = GlobalScale;
	}
	
	//Not exposed to Blueprint
	bool GetEnableLiveLinkEvaluation()
	{
		return GetProxyOnGameThread<FPCapPropLiveLinkAnimInstanceProxy>().PoseNode.bDoLiveLinkEvaluation;
	}

	protected:
	virtual FAnimInstanceProxy* CreateAnimInstanceProxy() override;
	virtual void DestroyAnimInstanceProxy(FAnimInstanceProxy* InProxy) override;

	friend FPCapPropLiveLinkAnimInstanceProxy;
	
};
