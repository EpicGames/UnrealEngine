// Copyright Epic Games, Inc. All Rights Reserved.


#include "PCapStageRoot.h"
#include "UObject/ConstructorHelpers.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Materials/MaterialInterface.h"



// Sets default values
APerformanceCaptureStageRoot::APerformanceCaptureStageRoot(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Set this actor to call Tick() every frame.  You can turn this off to improve performance if you don't need it.
	PrimaryActorTick.bCanEverTick = true;

	USceneComponent* SceneRoot = CreateDefaultSubobject<USceneComponent>("Root");
	SetRootComponent(SceneRoot);
	MapCaptureComponent = CreateDefaultSubobject<USceneCaptureComponent2D>("MapCaptureComponent");
	MapCaptureComponent->SetupAttachment(GetRootComponent());
	MapCaptureComponent->SetRelativeLocationAndRotation(FVector(0.0f, 0.0f, 500.0f), FRotator(-90.0f, 0.0f, -90.0f));
	
	// Set the default properties we want on the map scene capture. By default, do not run on every frame
	MapCaptureComponent->bCaptureEveryFrame = false;
	MapCaptureComponent->bCaptureOnMovement = true;
	MapCaptureComponent->CaptureSource = SCS_BaseColor;
	MapCaptureComponent->ProjectionType = ECameraProjectionMode::Orthographic;
	MapCaptureComponent->OrthoWidth = 1024.0f;
	MapCaptureComponent->bUpdateOrthoPlanes = true;
	
	// Load and reference a render target in the plugin content
	ConstructorHelpers::FObjectFinder<UTextureRenderTarget2D> MapTexture(TEXT("/PerformanceCaptureWorkflow/Stage/RT_OrthoView.RT_OrthoView"));
	if (MapTexture.Succeeded())
	{
		MapCaptureComponent->TextureTarget = MapTexture.Object;
	}

	DecalComponent = CreateDefaultSubobject<UDecalComponent>("Decal");
	DecalComponent->SetupAttachment(GetRootComponent());
	DecalComponent->SetRelativeLocationAndRotation(FVector(0.0f, 0.0f, -100.0f), FRotator(0.0f, 0.0f, -90.0f));
	
	// Load and reference the 1m grid decal material
	ConstructorHelpers::FObjectFinder<UMaterialInterface> DecalMaterial(TEXT("/PerformanceCaptureWorkflow/Stage/MI_ProcGrid.MI_ProcGrid"));
	if (DecalMaterial.Succeeded())
	{
		DecalComponent->SetDecalMaterial(DecalMaterial.Object);
	}
	
	DecalComponent->DecalSize = FVector(256.0f, 256.0f, 256.0f);
	DecalComponent->SetVisibility(false);

	// Create a scene component to which all the stage meshes can be parented
	StageMeshParent = CreateDefaultSubobject<USceneComponent>("StageMeshParent");
	StageMeshParent->SetupAttachment(GetRootComponent());
}