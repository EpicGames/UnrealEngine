// Copyright Epic Games, Inc. All Rights Reserved.


#include "PCapWorkflowRuntimeFunctionLibrary.h"

#include "PerformanceCaptureWorkflowRuntime.h"
#include "Retargeter/IKRetargeter.h"
#include "Retargeter/RetargetOps/FKChainsOp.h"

const UIKRigDefinition* UPCapWorkflowRuntimeFunctionLibrary::GetSourceRig(const UIKRetargeter* InRetargetAsset)
{
	if (InRetargetAsset)
	{
		return InRetargetAsset->GetIKRig(ERetargetSourceOrTarget::Source);
	}
	return nullptr;
}

const UIKRigDefinition* UPCapWorkflowRuntimeFunctionLibrary::GetTargetRig(const UIKRetargeter* InRetargetAsset)
{
	if (InRetargetAsset)
	{
		return InRetargetAsset->GetIKRig(ERetargetSourceOrTarget::Target);
	}
	return nullptr;
}

TArray<FBoneChain> UPCapWorkflowRuntimeFunctionLibrary::GetRetargetChains(const UIKRigDefinition* InIKRig)
{
	if (InIKRig)
	{
		return InIKRig->GetRetargetChains();
	}
	return TArray<FBoneChain>();
}

FName UPCapWorkflowRuntimeFunctionLibrary::GetChainStartBone(const UIKRigDefinition* InIKRig, const FName ChainName)
{
	FName FoundBone = NAME_None;
	if (InIKRig)
	{
		const FBoneChain* Chain = InIKRig->GetRetargetChainByName(ChainName);
		if (!Chain)
		{
			UE_LOGF(LogPCapRuntime, Verbose, "Retarget chain not found: %ls.", *ChainName.ToString())
			return NAME_None;
		}
		FoundBone = Chain->StartBone.BoneName;
	}
	return FoundBone;
}

FName UPCapWorkflowRuntimeFunctionLibrary::GetChainEndBone(const UIKRigDefinition* InIKRig, const FName ChainName)
{
	FName FoundBone = NAME_None;
	if (InIKRig)
	{
		const FBoneChain* Chain = InIKRig->GetRetargetChainByName(ChainName);
		if (!Chain)
		{
			UE_LOGF(LogPCapRuntime, Verbose, "Retarget chain not found: %ls.", *ChainName.ToString())
			return NAME_None;
		}
		FoundBone = Chain->EndBone.BoneName;
	}
	return FoundBone;
}

FName UPCapWorkflowRuntimeFunctionLibrary::GetChainFromBone(const UIKRigDefinition* InIKRig, const FName InBoneName)
{
	FName ChainName = NAME_None;
	
	if (InIKRig)
	{
		TArray<FBoneChain> Chains = InIKRig->GetRetargetChains();

		for (FBoneChain Chain : Chains)
		{
			if (InBoneName == Chain.StartBone.BoneName || InBoneName == Chain.EndBone.BoneName)
			{
				ChainName = Chain.ChainName;
				return ChainName;
			}
		}
	}
	return ChainName;
}

FName UPCapWorkflowRuntimeFunctionLibrary::GetChainPair(UIKRetargeter* InRetargetAsset, const FName InChainName,
                                                        const ERetargetSourceOrTarget SourceOrTarget)
{
	FName ChainName = NAME_None;

	if (InRetargetAsset && InChainName!=NAME_None)
	{
		if (const FIKRetargetFKChainsOp* FKOp = InRetargetAsset->GetFirstRetargetOpOfType<FIKRetargetFKChainsOp>())
		{
			const FIKRetargetFKChainsOpSettings& ChainsSettings = FKOp->Settings;
			if (const FRetargetChainPair* ChainPair = ChainsSettings.ChainMapping.FindChainPair(InChainName,SourceOrTarget))
			{
				ChainName = ChainPair->SourceChainName;
			}
		}
	}
	return ChainName;
}
