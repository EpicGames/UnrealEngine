// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "PCapWorkflowRuntimeFunctionLibrary.generated.h"

/**
 * Function Library for Performance Capture Workflow, enabling read-only access to Retargeter properties.
 * Intended for querying retargeters when calculating dynamic prop constraints. 
 */
UCLASS()
class PERFORMANCECAPTUREWORKFLOWRUNTIME_API UPCapWorkflowRuntimeFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
	/**
	 * Get the source IK rig from an IKRetargeter asset.
	 */
	UFUNCTION(BlueprintCallable, Category="Performance Capture|Runtime")
	static const UIKRigDefinition* GetSourceRig(const UIKRetargeter* InRetargetAsset);

	/**
	 * Get the target IK rig from an IKRetargeter asset.
	 */
	UFUNCTION(BlueprintCallable, Category="Performance Capture|Runtime")
	static const UIKRigDefinition* GetTargetRig(const UIKRetargeter* InRetargetAsset);

	/**
	 * Get all the chains in the given IKRig Asset.
	 */
	UFUNCTION(BlueprintCallable, Category="Performance Capture|Runtime")
	static TArray<FBoneChain> GetRetargetChains(const UIKRigDefinition* InIKRig);

	/**
	 * Returns the bone name at the start of a given chain.
	 */
	UFUNCTION(BlueprintCallable, Category="Performance Capture|Runtime")
	static FName GetChainStartBone(const UIKRigDefinition* InIKRig, const FName ChainName);

	/**
	 * Returns the bone name at the end of a given chain.
	 */
	UFUNCTION(BlueprintCallable, Category="Performance Capture|Runtime")
	static FName GetChainEndBone(const UIKRigDefinition* InIKRig, const FName ChainName);

	/**
	 * Returns the first chain the given bone name is a member of.
	 */
	UFUNCTION(BlueprintCallable, Category="Performance Capture|Runtime")
	static FName GetChainFromBone(const UIKRigDefinition* InIKRig, const FName InBoneName);

	/**
	 * Get the corresponding source or target chain from a given chain name.
	 */
	UFUNCTION(BlueprintCallable, Category="Performance Capture|Runtime")
	static FName GetChainPair(UIKRetargeter* InRetargetAsset, const FName InChainName, const ERetargetSourceOrTarget SourceOrTarget);
};
