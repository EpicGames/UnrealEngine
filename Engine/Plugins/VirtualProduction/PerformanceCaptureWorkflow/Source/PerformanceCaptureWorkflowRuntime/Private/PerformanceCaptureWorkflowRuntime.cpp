// Copyright Epic Games, Inc. All Rights Reserved.
#include "PerformanceCaptureWorkflowRuntime.h"

#define LOCTEXT_NAMESPACE "FPerformanceCaptureWorkflowRuntimeModule"

DEFINE_LOG_CATEGORY(LogPCapRuntime);

void FPerformanceCaptureWorkflowRuntimeModule::StartupModule()
{
	
}

void FPerformanceCaptureWorkflowRuntimeModule::ShutdownModule()
{
    
}

#undef LOCTEXT_NAMESPACE
    
IMPLEMENT_MODULE(FPerformanceCaptureWorkflowRuntimeModule, PerformanceCaptureWorkflowRuntime)