// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ILiveLinkClient.h"
#include "Components/SceneComponent.h"
#include "Components/ActorComponent.h"
#include "Components/StaticMeshComponent.h"
#include "PCapPropComponent.generated.h"

class ACaptureCharacter;
/**
* A component for use in Motion Capture props. Accepts data as either a LiveLink Transform Role or an Animation Role. If the Live Link data is in the Animation role, this component will take the root bone transform and apply that transform to the owning actor's root component. Can be applied to Static Meshes, Skeletal Meshes and Blueprint constructions. Transform data can be offset in the component's local space. If this component is driving a Skeletal Mesh component and is receiving Live Link Animation data the full bone hierarchy will be applied. 
*/
UCLASS(Blueprintable, ClassGroup=("Performance Capture"), meta=(BlueprintSpawnableComponent, DisplayName="Prop Component"))
class PERFORMANCECAPTUREWORKFLOWRUNTIME_API UPCapPropComponent : public UActorComponent
{
	GENERATED_BODY()

public:
	// Constructor
	UPCapPropComponent();
	// Destructor
	~UPCapPropComponent();

	/**
	 * LiveLink Subject Name. Must have either Animation Role Type.
	 */
	UPROPERTY(EditAnywhere, Category = "Performance Capture")
	FLiveLinkSubjectName SubjectName;

	/**
	 * Should LiveLink Subject data be evaluated.
	 */
	UPROPERTY(EditAnywhere, Category = "Performance Capture")
	bool bEvaluateLiveLink = true;

	/**
	 * Overrides all LiveLink data and the offset transform. Must be manually set on possessable bindings.
	 */
	UPROPERTY(BlueprintReadOnly, EditInstanceOnly, Interp, Category = "Performance Capture")
	bool bIsControlledBySequencer = false;

	/**
	 * The package name of the spawning data asset, if there is one. 
	 */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Performance Capture")
	FName SpawningDataAsset;
	
	/**
	 * Offset the incoming LiveLink Transform data in the local space of the controlled component.
	 */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Performance Capture")
	FTransform OffsetTransform = FTransform::Identity;
	
	/**
	 * Scale the transform from the parent pivot (usually the motion capture stage origin point).
	 * Scale is only applied to the location, not the scale of the driven mesh. 
	 */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Performance Capture")
	float GlobalScale = 1.0f;

	/**
	 * Scene Component driven by the LiveLink data. Defaults to Root component of the owning actor.
	 */
	UPROPERTY(EditInstanceOnly, Category = "Performance Capture", meta=(UseComponentPicker, AllowedClasses = "/Script/Engine.SceneComponent"))
	FComponentReference ControlledComponent;

	/**
	 * Returns the component controlled by this component. Can return null.
	 */
	UFUNCTION(BlueprintPure, Category = "Performance Capture")
	USceneComponent* GetControlledComponent() const;

	/**
	 * Sets the component controlled by this component. Component must be within the same actor as this Prop component.
	 */
	UFUNCTION(BlueprintCallable, Category = "Performance Capture")
	void SetControlledComponent(USceneComponent* InComponent);

	/**
	 * Set the Live Link subject used by this prop component. 
	 * @param Subject Live Link Subject Name
	 */
	UFUNCTION(BlueprintCallable, Category = "Performance Capture|Prop")
	void SetLiveLinkSubject(FLiveLinkSubjectName Subject);
	
	/**
	* Get the LiveLink Subject Name.
	* @return FLiveLinkSubjectName Current LiveLink Subject.
	*/
	UFUNCTION(BlueprintCallable, Category ="Performance Capture|Prop")
	FLiveLinkSubjectName GetLiveLinkSubject() const;

	/**
	* Set the LiveLink data to update the Skeletal Mesh pose.
	* @param bEvaluateLinkLink Drive or pause the Controlled Mesh from LiveLink Subject data.
	*/
	UFUNCTION(BlueprintCallable, Category ="Performance Capture|Prop")
	void SetEvaluateLiveLinkData(bool bEvaluateLinkLink);
	
	/**
	* Get the LiveLink Evaluation State. Subject must have the Animation Role Type.
	* @return bool Is LiveLink data being evaluated.
	*/
	UFUNCTION(BlueprintPure, Category ="Performance Capture|Prop")
	bool GetEvaluateLiveLinkData();
	
	/**
	* Update the local space offset to this prop's Live Link pose. 
	* @param NewOffset Offset Transform.
	*/
	UFUNCTION(BlueprintCallable, Category ="Performance Capture|Prop")
	void SetOffsetTransform(FTransform NewOffset);
	
	/**
	* Update the scale factor applied to the incoming Live Link poses 
	* @param NewGlobalScale Global Scale.
	*/
	UFUNCTION(BlueprintCallable, Category ="Performance Capture|Prop")
	void SetGlobalScale(float NewGlobalScale);

	/**
	* Enable/Disable use of the Dynamic Constraint offset vector. 
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category ="Performance Capture|Dynamic Constraint")
	bool bUseDynamicConstraint = false;
	
	/**
	* List of Capture Characters to consider in calculating a dynamic constraint.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Performance Capture|Dynamic Constraint", meta = (EditCondition="bUseDynamicConstraint", EditConditionHides))
	TArray<TSoftObjectPtr<ACaptureCharacter>> DynamicAttachmentCharacters;

	/**
	* False = All bone chains' start and end bones will be considered for dynamic attachment .
	* True  = Only bones in chains with an IK goal will be considered for dynamic attachment.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Performance Capture|Dynamic Constraint", meta = (EditCondition="bUseDynamicConstraint", EditConditionHides))
	bool bAllowOnlyIKBones = true;

	/**
	* A list of bones of to test against for dynamic constraint.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Performance Capture|Dynamic Constraint", meta = (GetOptions="GetAttachBones", EditCondition="bUseDynamicConstraint", EditConditionHides))
	TArray<FName> DynamicConstraintAttachBones;

	/**
	 * Return a list of bones from the given list of DynamicAttachmentCharacters.
	 * @return List of bones to use in DynamicConstraintAttachBones variable
	 */
	UFUNCTION()
	TArray<FName> GetAttachBones() const;

	/**
	 * Blueprint implementable event to enable calculation of a location offset before applying it to
	 * the controlled component's transform.
	 * @return Parent-space location offset to apply to the prop's transform
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "Performance Capture|Dynamic Constraint")
	FVector CalculateDynamicOffset();
	
protected:

	//~ Begin UActorComponent interface
	virtual void OnRegister() override;
	virtual void DestroyComponent(bool bPromoteChildren) override;
	//~ End UActorComponent interface
	
	//~ UObject interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject interface
	

public:
	// Called every frame
	virtual void TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;

	void InitiateAnimation();

	void ResetAnimInstance() const;
	
private:
	FVector DynamicConstraintOffset = FVector::ZeroVector;
	
	FTransform CachedLiveLinkTransform = FTransform::Identity;
	
	FTransform FinalTransform = FTransform::Identity;

	bool bIsDirty;

	TOptional<bool> bIsSpawnableCache;
};
