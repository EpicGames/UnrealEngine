// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "Components/SceneComponent.h"
#include "Components/DecalComponent.h"
#include "Components/SceneCaptureComponent2D.h"

#include "PCapStageRoot.generated.h"

class USceneCaptureComponent2D;

UCLASS(Blueprintable, Abstract)
class APerformanceCaptureStageRoot : public AActor
{
	GENERATED_BODY()

public:
	// Sets default values for this actor's properties
	APerformanceCaptureStageRoot(const FObjectInitializer& ObjectInitializer);

	/** Scene capture component for capturing the overhead, orthographic map. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite,  Category="Peformance Capture|StageRoot")
	TObjectPtr<USceneCaptureComponent2D> MapCaptureComponent;

	/** Decal component for rendering a grid to the floor. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite,  Category="Peformance Capture|StageRoot")
	TObjectPtr<UDecalComponent> DecalComponent;

	/** Scene component, under which all stage ghost meshes should be parented. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite,  Category="Peformance Capture|StageRoot")
	TObjectPtr<USceneComponent> StageMeshParent;
	
};